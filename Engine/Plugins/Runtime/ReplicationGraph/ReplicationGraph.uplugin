{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Replication Graph",
	"Description": "The Replication Graph plugin provides a Replication Driver implementation designed for a large number of actors and connections by mainting persistent replicated actor lists in a graph structure.",
	"Category": "Performance",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ReplicationGraph",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"IsExperimentalVersion": false
}