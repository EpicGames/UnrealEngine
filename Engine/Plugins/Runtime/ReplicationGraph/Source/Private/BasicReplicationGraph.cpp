// Copyright Epic Games, Inc. All Rights Reserved.
// 
#include "BasicReplicationGraph.h"
#include "UObject/UObjectIterator.h"
#include "Engine/ChildConnection.h"

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameFramework/PlayerController.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicReplicationGraph)

UBasicReplicationGraph::UBasicReplicationGraph()
{

}

void UBasicReplicationGraph::InitGlobalActorClassSettings()
{
	Super::InitGlobalActorClassSettings();

	// ReplicationGraph stores internal associative data for actor classes. 
	// We build this data here based on actor CDO values.
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		AActor* ActorCDO = Cast<AActor>(Class->GetDefaultObject());
		if (!ActorCDO || !ActorCDO->GetIsReplicated())
		{
			continue;
		}

		// Skip SKEL and REINST classes.
		if (Class->GetName().StartsWith(TEXT("SKEL_")) || Class->GetName().StartsWith(TEXT("REINST_")))
		{
			continue;
		}

		FClassReplicationInfo ClassInfo;

		// Replication Graph is frame based. Convert NetUpdateFrequency to ReplicationPeriodFrame based on Server MaxTickRate.
		ClassInfo.ReplicationPeriodFrame = GetReplicationPeriodFrameForFrequency(ActorCDO->GetNetUpdateFrequency());
		
		if (ActorCDO->bAlwaysRelevant || ActorCDO->bOnlyRelevantToOwner)
		{
			ClassInfo.SetCullDistanceSquared(0.f);
		}
		else
		{
			ClassInfo.SetCullDistanceSquared(ActorCDO->GetNetCullDistanceSquared());
		}
		
		GlobalActorReplicationInfoMap.SetClassInfo( Class, ClassInfo );
	}

#if WITH_GAMEPLAY_DEBUGGER
	AGameplayDebuggerCategoryReplicator::NotifyDebuggerOwnerChange.AddUObject(this, &ThisClass::OnGameplayDebuggerOwnerChange);
#endif
}

void UBasicReplicationGraph::InitGlobalGraphNodes()
{
	// -----------------------------------------------
	//	Spatial Actors
	// -----------------------------------------------

	GridNode = CreateNewNode<UReplicationGraphNode_GridSpatialization2D>();
	GridNode->CellSize = 10000.f;
	GridNode->SpatialBias = FVector2D(-UE_OLD_WORLD_MAX, -UE_OLD_WORLD_MAX);

	AddGlobalGraphNode(GridNode);

	// -----------------------------------------------
	//	Always Relevant (to everyone) Actors
	// -----------------------------------------------
	AlwaysRelevantNode = CreateNewNode<UReplicationGraphNode_ActorList>();
	AddGlobalGraphNode(AlwaysRelevantNode);
}

void UBasicReplicationGraph::InitConnectionGraphNodes(UNetReplicationGraphConnection* RepGraphConnection)
{
	Super::InitConnectionGraphNodes(RepGraphConnection);

	UReplicationGraphNode_AlwaysRelevant_ForConnection* AlwaysRelevantNodeForConnection = CreateNewNode<UReplicationGraphNode_AlwaysRelevant_ForConnection>();
	AddConnectionGraphNode(AlwaysRelevantNodeForConnection, RepGraphConnection);

	AlwaysRelevantForConnectionList.Emplace(RepGraphConnection->NetConnection, AlwaysRelevantNodeForConnection);
}

void UBasicReplicationGraph::RouteAddNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo)
{
#if WITH_GAMEPLAY_DEBUGGER
	// This is intended as an example and not shipping code, see the Lyra sample for a better way to handle routing with class routing mappings.
	if (ActorInfo.Actor->IsA(AGameplayDebuggerCategoryReplicator::StaticClass()))
	{
		return;
	}
#endif

	ensureMsgf((ActorInfo.Actor->bAlwaysRelevant && ActorInfo.Actor->bOnlyRelevantToOwner) == false, TEXT("Replicated actor %s is both bAlwaysRelevant and bOnlyRelevantToOwner. Only one can be supported."), *ActorInfo.Actor->GetName());
	if (ActorInfo.Actor->bAlwaysRelevant)
	{
		AlwaysRelevantNode->NotifyAddNetworkActor(ActorInfo);
	}
	else if (ActorInfo.Actor->bOnlyRelevantToOwner)
	{
		ActorsWithoutNetConnection.Add(ActorInfo.Actor);
	}
	else
	{
		// Note that UReplicationGraphNode_GridSpatialization2D has 3 methods for adding actor based on the mobility of the actor. Since AActor lacks this information, we will
		// add all spatialized actors as dormant actors: meaning they will be treated as possibly dynamic (moving) when not dormant, and as static (not moving) when dormant.
		GridNode->AddActor_Dormancy(ActorInfo, GlobalInfo);
	}
}

void UBasicReplicationGraph::RouteRemoveNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo)
{
#if WITH_GAMEPLAY_DEBUGGER
	// This is intended as an example and not shipping code, see the Lyra sample for a better way to handle routing with class routing mappings.
	if (ActorInfo.Actor->IsA(AGameplayDebuggerCategoryReplicator::StaticClass()))
	{
		return;
	}
#endif

	if (ActorInfo.Actor->bAlwaysRelevant)
	{
		AlwaysRelevantNode->NotifyRemoveNetworkActor(ActorInfo);
		SetActorDestructionInfoToIgnoreDistanceCulling(ActorInfo.GetActor());
	}
	else if (ActorInfo.Actor->bOnlyRelevantToOwner)
	{
		if (UReplicationGraphNode* Node = ActorInfo.Actor->GetNetConnection() ? GetAlwaysRelevantNodeForConnection(ActorInfo.Actor->GetNetConnection()) : nullptr)
		{
			Node->NotifyRemoveNetworkActor(ActorInfo);
		}
	}
	else
	{
		GridNode->RemoveActor_Dormancy(ActorInfo);
	}
}

void UBasicReplicationGraph::RouteRenameNetworkActorToNodes(const FRenamedReplicatedActorInfo& ActorInfo)
{
#if WITH_GAMEPLAY_DEBUGGER
	if (ActorInfo.NewActorInfo.Actor->IsA(AGameplayDebuggerCategoryReplicator::StaticClass()))
	{
		return;
	}
#endif

	if (ActorInfo.NewActorInfo.Actor->bAlwaysRelevant)
	{
		AlwaysRelevantNode->NotifyActorRenamed(ActorInfo);
	}
	else if (ActorInfo.NewActorInfo.Actor->bOnlyRelevantToOwner)
	{
		if (UReplicationGraphNode* Node = ActorInfo.NewActorInfo.Actor->GetNetConnection() ? GetAlwaysRelevantNodeForConnection(ActorInfo.NewActorInfo.Actor->GetNetConnection()) : nullptr)
		{
			Node->NotifyActorRenamed(ActorInfo);
		}
	}
	else
	{
		GridNode->RenameActor_Dormancy(ActorInfo);
	}
}

UReplicationGraphNode_AlwaysRelevant_ForConnection* UBasicReplicationGraph::GetAlwaysRelevantNodeForConnection(UNetConnection* Connection)
{
	UReplicationGraphNode_AlwaysRelevant_ForConnection* Node = nullptr;
	if (Connection)
	{
		if (FConnectionAlwaysRelevantNodePair* Pair = AlwaysRelevantForConnectionList.FindByKey(Connection))
		{
			if (Pair->Node)
			{
				Node = Pair->Node;
			}
			else
			{
				UE_LOGF(LogNet, Warning, "AlwaysRelevantNode for connection %ls is null.", *GetNameSafe(Connection));
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Could not find AlwaysRelevantNode for connection %ls. This should have been created in UBasicReplicationGraph::InitConnectionGraphNodes.", *GetNameSafe(Connection));
		}
	}
	else
	{
		// Basic implementation requires owner is set on spawn that never changes. A more robust graph would have methods or ways of listening for owner to change
		UE_LOGF(LogNet, Warning, "Actor: bOnlyRelevantToOwner is set but does not have an owning Netconnection. It will not be replicated");
	}

	return Node;
}

int32 UBasicReplicationGraph::ServerReplicateActors(float DeltaSeconds)
{
	// Route Actors needing owning net connections to appropriate nodes
	for (int32 idx=ActorsWithoutNetConnection.Num()-1; idx>=0; --idx)
	{
		bool bRemove = false;
		if (AActor* Actor = ActorsWithoutNetConnection[idx])
		{
			if (UNetConnection* Connection = Actor->GetNetConnection())
			{
				bRemove = true;
				if (UReplicationGraphNode_AlwaysRelevant_ForConnection* Node = GetAlwaysRelevantNodeForConnection(Actor->GetNetConnection()))
				{
					Node->NotifyAddNetworkActor(FNewReplicatedActorInfo(Actor));
				}
			}
		}
		else
		{
			bRemove = true;
		}

		if (bRemove)
		{
			ActorsWithoutNetConnection.RemoveAtSwap(idx, EAllowShrinking::No);
		}
	}


	return Super::ServerReplicateActors(DeltaSeconds);
}

#if WITH_GAMEPLAY_DEBUGGER
void UBasicReplicationGraph::OnGameplayDebuggerOwnerChange(AGameplayDebuggerCategoryReplicator* Debugger, APlayerController* OldOwner)
{
	if (!Debugger || (Debugger->GetWorld() != GetWorld()))
	{
		return;
	}

	FNewReplicatedActorInfo ActorInfo(Debugger);

	if (OldOwner)
	{
		if (UReplicationGraphNode* Node = OldOwner->GetNetConnection() ? GetAlwaysRelevantNodeForConnection(OldOwner->GetNetConnection()) : nullptr)
		{
			Node->NotifyRemoveNetworkActor(ActorInfo);
		}
	}

	if (APlayerController* NewOwner = Debugger->GetReplicationOwner())
	{
		if (UReplicationGraphNode* Node = NewOwner->GetNetConnection() ? GetAlwaysRelevantNodeForConnection(NewOwner->GetNetConnection()) : nullptr)
		{
			Node->NotifyAddNetworkActor(ActorInfo);
		}
	}
}
#endif // WITH_GAMEPALY_DEBUGGER

bool FConnectionAlwaysRelevantNodePair::operator==(UNetConnection* InConnection) const
{
	// Any children should be looking at their parent connections instead.
	if (InConnection->GetUChildConnection() != nullptr)
	{
		InConnection = ((UChildConnection*)InConnection)->Parent;
	}

	return InConnection == NetConnection;
}

