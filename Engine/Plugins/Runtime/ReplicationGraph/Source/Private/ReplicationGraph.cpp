// Copyright Epic Games, Inc. All Rights Reserved.
// 


/**
 *	
 *	===================== TODO / WIP Notes =====================
 *
 * TODO Missing Features:
 *	-bNetTemporary
 * 	 	
 *	--------------------------------
 *	
 *	Game Code API
 *	
 *	
 *	Function						Status (w/ RepDriver enabled)
 *	----------------------------------------------------------------------------------------
 *	ForceNetUpdate					Compatible/Working			
 *	FlushNetDormancy				Compatible/Working
 *	SetNetUpdateTime				NOOP							
 *	ForceNetRelevant				NOOP
 *	ForceActorRelevantNextUpdate	NOOP
 *
 *	FindOrAddNetworkObjectInfo		NOOP. Accessing legacy system data directly. This sucks and should never have been exposed to game code directly.
 *	FindNetworkObjectInfo			NOOP. Will want to deprecate both of these functions and get them out of our code base
 *	
 *	
 *	FastShared Path information:
 *	------------------------------
 *	-Bind FClassReplicationInfo::FastSharedReplicationFunc to a function. That function should call a NetMulticast, Unrelable RPC with some parameters.
 *	-Those parameters are your "fastshared" data. It should be a struct of shareable data (no UObject references of connection specific data).
 *	-Return actors of this type in a list with the EActorRepListTypeFlags::FastShared flag.
 *	-UReplicationGraphNode_ActorListFrequencyBuckets can do this. See ::GatherActorListsForConnection.
 *	-(You must opt in to this by setting UReplicationGraphNode_ActorListFrequencyBuckets::EnableFastPath=true)
 *
 */

#include "ReplicationGraph.h"

#include "Engine/ChildConnection.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/PackageMapClient.h"
#include "Engine/World.h"
#include "Net/DataReplication.h"
#include "Engine/ActorChannel.h"
#include "GameFramework/PlayerController.h"
#include "Net/RepLayout.h"
#include "Net/Core/Misc/NetCVars.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "Engine/NetworkObjectList.h"
#include "UObject/UObjectIterator.h"
#include "DrawDebugHelpers.h"
#include "Misc/ScopeExit.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Engine/ServerStatReplicator.h"
#include "Stats/StatsTrace.h"
#include <limits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationGraph)

#if USE_SERVER_PERF_COUNTERS
#endif

namespace UE::Net::Private
{
	// Maintain pre-large world bounds for now, since the GridSpatialization2D node
	// uses a densely-stored grid and risks very high memory usage at large coordinates.
	constexpr double RepGraphWorldMax = UE_OLD_WORLD_MAX;
	constexpr double RepGraphHalfWorldMax = UE_OLD_HALF_WORLD_MAX;

	constexpr int32 RepGraphDormantActorsListReservedBuffer = 2048;
	// TInlineAllocator's size is optimized for default DestroyDormantDynamicActorsCellTTL value
	constexpr int32 RepGraphDormancyNodesInlineBufferSize = 200;

	// Track time for CVar_RepGraph_LogDebugInfoPeriod. Even though it's static, will be reset when a new repgraph instance is created
	static std::atomic<double> LastTimeLoggedMoreInfo = 0;

	int32 CVar_RepGraph_HandleDynamicActorRename = 0;
	static FAutoConsoleVariableRef CVarRepGraphHandleDynamicActorRename(TEXT("Net.RepGraph.HandleDynamicActorRename"), CVar_RepGraph_HandleDynamicActorRename, TEXT("If nonzero, when a dynamic actor's outer/level changes, repgraph will update its cached level information."));
}

int32 CVar_RepGraph_Pause = 0;
static FAutoConsoleVariableRef CVarRepGraphPause(TEXT("Net.RepGraph.Pause"), CVar_RepGraph_Pause,
	TEXT("Pauses actor replication in the Replication Graph."), ECVF_Default);

int32 CVar_RepGraph_Frequency = 0;
static FAutoConsoleVariableRef CVarRepGraphFrequency(TEXT("Net.RepGraph.Frequency.Override"), CVar_RepGraph_Frequency,
	TEXT("Explicit override for actor replication frequency"), ECVF_Default);

int32 CVar_RepGraph_Frequency_MatchTargetInPIE = 1;
static FAutoConsoleVariableRef CVarRepGraphFrequencyMatchTargetInPIE(TEXT("Net.RepGraph.Frequency.MatchTargetInPIE"), CVar_RepGraph_Frequency_MatchTargetInPIE,
	TEXT("In PIE, repgraph will update at the UNetDriver::NetServerMaxTickRate rate"), ECVF_Default);

int32 CVar_RepGraph_UseLegacyBudget = 1;
static FAutoConsoleVariableRef CVarRepGraphUseLegacyBudget(TEXT("Net.RepGraph.UseLegacyBudget"), CVar_RepGraph_UseLegacyBudget,
	TEXT("Use legacy IsNetReady() to make dynamic packget budgets"), ECVF_Default);

float CVar_RepGraph_FixedBudget = 0;
static FAutoConsoleVariableRef CVarRepGraphFixedBudge(TEXT("Net.RepGraph.FixedBudget"), CVar_RepGraph_FixedBudget,
	TEXT("Set fixed (independent of frame rate) packet budget. In BIts/frame"), ECVF_Default);

int32 CVar_RepGraph_SkipDistanceCull = 0;
static FAutoConsoleVariableRef CVarRepGraphSkipDistanceCull(TEXT("Net.RepGraph.SkipDistanceCull"), CVar_RepGraph_SkipDistanceCull,
	TEXT("Debug option to skip distance culling during evaluation"), ECVF_Default);

int32 CVar_RepGraph_PrintCulledOnConnectionClasses = 0;
static FAutoConsoleVariableRef CVarRepGraphPrintCulledOnConnectionClasses(TEXT("Net.RepGraph.PrintCulledOnConnectionClasses"), CVar_RepGraph_PrintCulledOnConnectionClasses,
	TEXT("Debug option to print culling stats"), ECVF_Default);

int32 CVar_RepGraph_TrackClassReplication = 0;
static FAutoConsoleVariableRef CVarRepGraphTrackClassReplication(TEXT("Net.RepGraph.TrackClassReplication"), CVar_RepGraph_TrackClassReplication,
	TEXT("Debug option to track class replication stats"), ECVF_Default);

int32 CVar_RepGraph_NbDestroyedGridsToTriggerGC = 100;
static FAutoConsoleVariableRef CVarRepGraphNbDestroyedGridsToTriggerGC(TEXT("Net.RepGraph.NbDestroyedGridsToTriggerGC"), CVar_RepGraph_NbDestroyedGridsToTriggerGC,
	TEXT("After destroying this many grids, force a garbage collection to free memory"), ECVF_Default);

int32 CVar_RepGraph_PrintTrackClassReplication = 0;
static FAutoConsoleVariableRef CVarRepGraphPrintTrackClassReplication(TEXT("Net.RepGraph.PrintTrackClassReplication"), CVar_RepGraph_PrintTrackClassReplication,
	TEXT("Debug option to print class replication stats"), ECVF_Default);

int32 CVar_RepGraph_DormantDynamicActorsDestruction = 0;
static FAutoConsoleVariableRef CVarRepGraphDormantDynamicActorsDestruction(TEXT("Net.RepGraph.DormantDynamicActorsDestruction"), CVar_RepGraph_DormantDynamicActorsDestruction,
	TEXT("If true, irrelevant dormant actors will be destroyed on the client"), ECVF_Default);

int32 CVar_RepGraph_DormantCellsTTLDefault = 200;
static FAutoConsoleVariableRef CVarRepGraphDormantCellsTTLDefault(TEXT("Net.RepGraph.DormantCellsTTLDefault"), CVar_RepGraph_DormantCellsTTLDefault,
	TEXT("Determine number of frames cell will be considered visible before dormant dynamic actor destruction kicks in"), ECVF_Default);

int32 CVar_RepGraph_ReplicatedDormantDestructionInfosPerFrame = MAX_int32;
static FAutoConsoleVariableRef CVarRepGraphReplicatedDormantDestructionInfosPerFrame(TEXT("Net.RepGraph.ReplicatedDormantDestructionInfosPerFrame"), CVar_RepGraph_ReplicatedDormantDestructionInfosPerFrame,
	TEXT("If CVarRepGraphDormantDynamicActorsDestruction is true, this is the max number of destruction infos sent to a client per frame"), ECVF_Default);

int32 CVar_RepGraph_MarkDormantDestroyedActorsActive = 1;
static FAutoConsoleVariableRef CVarRepGraphMarkDormantDestroyedActorsActive(TEXT("Net.RepGraph.MarkDormantDestroyedActorsActive"), CVar_RepGraph_MarkDormantDestroyedActorsActive,
	TEXT("If CVarRepGraphDormantDynamicActorsDestruction is true, mark destroyed dormant actors active to keep their dormancy state with the NetDriver in sync"), ECVF_Default);

float CVar_RepGraph_OutOfRangeDistanceCheckRatio = 0.5f;
static FAutoConsoleVariableRef CVarRepGraphOutOfRangeDistanceCheckRatio(TEXT("Net.RepGraph.OutOfRangeDistanceCheckRatio"), CVar_RepGraph_OutOfRangeDistanceCheckRatio,
	TEXT("The ratio of DestructInfoMaxDistance that gives the distance traveled before we reevaluate the out of range destroyed actors list"), ECVF_Default);

int32 CVar_RepGraph_DormancyNode_ObsoleteBehavior = 0;
static FAutoConsoleVariableRef CVarRepGraphDormancyNodeObsoleteBehavior(TEXT("Net.RepGraph.DormancyNodeObsoleteBehavior"), CVar_RepGraph_DormancyNode_ObsoleteBehavior, TEXT("This changes how the dormancy node deals with obsolete nodes. 0 = ignore. 1 = lazily destroy the node"), ECVF_Default);

int32 CVar_RepGraph_GridSpatialization2D_DestroyDormantDynamicActorsDefault = 1;
static FAutoConsoleVariableRef CVarRepGraphGridSpatialization2DDestroyDormantDynamicActorsDefault(TEXT("Net.RepGraph.GridSpatialization2DDestroyDormantDynamicActorsDefault"), CVar_RepGraph_GridSpatialization2D_DestroyDormantDynamicActorsDefault, TEXT("Configure what the default for UReplicationGraphNode_GridSpatialization2D::DestroyDormantDynamicActors should be."), ECVF_Default);

int32 CVar_RepGraph_ConnectionHeavyComputationAmortization = 0;
static FAutoConsoleVariableRef CVarRepGraphConnectionHeavyComputationAmortization(TEXT("Net.RepGraph.ConnectionHeavyComputationAmortization"), CVar_RepGraph_ConnectionHeavyComputationAmortization, TEXT("Non zero values enable heavy computation cost amortization by updating one connection per frame."), ECVF_Default);

int32 CVar_RepGraph_LogDebugInfoPeriod = 60 * 5;
static FAutoConsoleVariableRef CVarRepGraphLogDebugInfoPeriod(TEXT("Net.RepGraph.LogDebugInfoPeriod"), CVar_RepGraph_LogDebugInfoPeriod,
	TEXT("How long to wait between logging debug info on invalid actors."), ECVF_Default);


REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.LogNetDormancyDetails", CVar_RepGraph_LogNetDormancyDetails, 0, "Logs actors that are removed from the replication graph/nodes.");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.LogActorRemove", CVar_RepGraph_LogActorRemove, 0, "Logs actors that are removed from the replication graph/nodes.");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.LogActorAdd", CVar_RepGraph_LogActorAdd, 0, "Logs actors that are added to replication graph/nodes.");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.Verify", CVar_RepGraph_Verify, 0, "Additional, slow, verification is done on replication graph nodes. Guards against: invalid actors and dupes");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.DisableBandwithLimit", CVar_RepGraph_DisableBandwithLimit, 0, "Disables the IsNetReady() check, effectively replicating all actors that want to replicate to each connection.");

REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.TrickleDistCullOnDormancyNodes", CVar_RepGraph_TrickleDistCullOnDormancyNodes, 1, "Actors in a dormancy node that are distance culled will trickle through as dormancy node empties");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.EnableRPCSendPolicy", CVar_RepGraph_EnableRPCSendPolicy, 1, "Enables RPC send policy (e.g, force certain functions to send immediately rather than be queued)");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.EnableFastSharedPath", CVar_RepGraph_EnableFastSharedPath, 1, "Enables FastShared replication path for lists with EActorRepListTypeFlags::FastShared flag");
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.EnableDynamicAllocationWarnings", CVar_RepGraph_EnableDynamicAllocationWarnings, 1, "Enables debug information whenever RepGraph needs to allocate new Actor Lists.");

DECLARE_STATS_GROUP(TEXT("ReplicationDriver"), STATGROUP_RepDriver, STATCAT_Advanced);
DECLARE_DWORD_COUNTER_STAT(TEXT("Rep Actor List Dupes"), STAT_NetRepActorListDupes, STATGROUP_RepDriver);
DECLARE_DWORD_COUNTER_STAT(TEXT("Actor Channels Opened"), STAT_NetActorChannelsOpened, STATGROUP_RepDriver);
DECLARE_DWORD_COUNTER_STAT(TEXT("Actor Channels Closed"), STAT_NetActorChannelsClosed, STATGROUP_RepDriver);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Processed Connections"), STAT_NumProcessedConnections, STATGROUP_RepDriver);

CSV_DEFINE_CATEGORY(ReplicationGraphMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphKBytes, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphChannelsOpened, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphNumReps, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphVisibleLevels, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphForcedUpdates, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphCleanMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphCleanNumReps, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(ReplicationGraphRedundantMS, WITH_SERVER_CODE);

CSV_DEFINE_CATEGORY(ReplicationGraph, WITH_SERVER_CODE);

static TAutoConsoleVariable<FString> CVarRepGraphConditionalBreakpointActorName(TEXT("Net.RepGraph.ConditionalBreakpointActorName"), TEXT(""), 
	TEXT("Helper CVar for debugging. Set this string to conditionally log/breakpoint various points in the repgraph pipeline. Useful for bugs like 'why is this actor channel closing'"), ECVF_Default );

// Variable that can be programatically set to a specific actor/connection 
FActorConnectionPair DebugActorConnectionPair;

/** Used to call Describe on a Connection or Channel, handling the null case. */
template<typename T>
static FORCEINLINE FString DescribeSafe(T* Describable)
{
	return Describable ? Describable->Describe() : FString(TEXT("None"));
}

FORCEINLINE bool RepGraphConditionalActorBreakpoint(AActor* Actor, UNetConnection* NetConnection)
{
#if !(UE_BUILD_SHIPPING)
	if (CVarRepGraphConditionalBreakpointActorName.GetValueOnGameThread().Len() > 0 && GetNameSafe(Actor).Contains(CVarRepGraphConditionalBreakpointActorName.GetValueOnGameThread()))
	{
		return true;
	}

	// Alternatively, DebugActorConnectionPair can be set by code to catch a specific actor/connection pair 
	if (DebugActorConnectionPair.Actor.Get() == Actor && (DebugActorConnectionPair.Connection == nullptr || DebugActorConnectionPair.Connection == NetConnection ))
	{
		return true;
	}
#endif
	return false;
}

// CVar that can be set to catch actor channel open/closing problems. This catches if we open/close actor channels for the same actor/connection pair too many times.
REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.ActorChannelWarnThreshold", CVar_RepGraph_WarnThreshold, 0, "");
TMap<FActorConnectionPair, int32> ActorChannelCreateCounter;
TMap<FActorConnectionPair, int32> ActorChannelDestroyCounter;

#if WITH_SERVER_CODE
static TAutoConsoleVariable<FString> CVarRepGraphConditionalPairActorName(TEXT("Net.RepGraph.ConditionalPairName"), TEXT(""), TEXT(""), ECVF_Default );
void UpdateActorConnectionCounter(AActor* InActor, UNetConnection* InConnection, TMap<FActorConnectionPair, int32>& Counter)
{
#if !(UE_BUILD_SHIPPING)
	if (CVar_RepGraph_WarnThreshold <= 0)
	{
		return;
	}

	if (CVarRepGraphConditionalPairActorName.GetValueOnGameThread().Len() > 0 && !GetNameSafe(InActor).Contains(CVarRepGraphConditionalPairActorName.GetValueOnGameThread()))
	{
		return;
	}

	if (DebugActorConnectionPair.Actor.IsValid() == false)
	{
		int32& Count = Counter.FindOrAdd( FActorConnectionPair(InActor, InConnection) );
		Count++;
		if (Count > CVar_RepGraph_WarnThreshold)
		{
			UE_LOGF(LogReplicationGraph, Warning, "Setting WARN Pair: %ls - %ls", *GetPathNameSafe(InActor), *InConnection->Describe());
			DebugActorConnectionPair = FActorConnectionPair(InActor, InConnection);
		}
	}
#endif
}
#endif // WITH_SERVER_CODE

/** Called by IsActorValidForReplication_LogMoreInfo to print more info on a failure */
void LogMoreInfoOnIsActorValidFailure(const FActorRepListType& In)
{
#if WITH_SERVER_CODE
	using namespace UE::Net::Private;

	double CurrentTime = FPlatformTime::Seconds();
	if (CVar_RepGraph_LogDebugInfoPeriod > 0 && ((CurrentTime - LastTimeLoggedMoreInfo) > double(CVar_RepGraph_LogDebugInfoPeriod)))
	{
		LastTimeLoggedMoreInfo = CurrentTime;

		AActor* Actor = In;
		if (Actor)
		{
			// Actor pointer is valid, but some of its properties may not be.
			UE_LOGF(LogReplicationGraph, Error, "Actor not valid for replication (BeingDestroyed:%d) (IsValid:%d) (Unreachable:%d) (TearOff:%d)! Actor = %ls",
				Actor->IsActorBeingDestroyed(), IsValid(Actor), Actor->IsUnreachable(), Actor->GetTearOff(),
				*Actor->GetFullName());
#if UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY
			UE_LOGF(LogReplicationGraph, Error, "More info about invalid actor '%ls': Owner='%ls', OuterPackage='%ls'",
				*In.ActorName.ToString(), *In.OwnerName.ToString(), *In.OuterPackageName.ToString());
#endif

			GEngine->Exec(Actor->GetWorld(), TEXT("Net.RepGraph.PrintAll"));
		}
		else
		{
#if UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY
			UE_LOGF(LogReplicationGraph, Error, "Invalid actor pointer detected during replication: Name='%ls', Owner='%ls', OuterPackage='%ls'",
				*In.ActorName.ToString(), *In.OwnerName.ToString(), *In.OuterPackageName.ToString()
				);
#else
			UE_LOGF(LogReplicationGraph, Error, "Invalid actor pointer detected during replication");
#endif
		}

		UE_LOGF(LogReplicationGraph, Log, "Invalid actor found at the last moment, printing (game-specific) current routing:");
		for (TObjectIterator<UReplicationGraph> It; It; ++It)
		{
			It->PrintGraphDebugInfo_OnInvalidActor();
		}
	}
#endif // WITH_SERVER_CODE
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

TFunction<void(int32)> UReplicationGraph::OnListRequestExceedsPooledSize;

UReplicationGraph::UReplicationGraph()
{
	ReplicationConnectionManagerClass = UNetReplicationGraphConnection::StaticClass();
	GlobalActorChannelFrameNumTimeout = 2;
	ActorDiscoveryMaxBitsPerFrame = 0;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		GraphGlobals = MakeShared<FReplicationGraphGlobalData>();
		GraphGlobals->GlobalActorReplicationInfoMap = &GlobalActorReplicationInfoMap;
		GraphGlobals->ReplicationGraph = this;
	}

	// Rebindable function for handling rep list requests that exceed preallocated pool size
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!OnListRequestExceedsPooledSize)
	{
		OnListRequestExceedsPooledSize = [](int32 NewExpectedSize)
		{
			if (CVar_RepGraph_EnableDynamicAllocationWarnings)
			{
				FReplicationGraphDebugInfo DebugInfo(*GLog);
				DebugInfo.Flags = FReplicationGraphDebugInfo::ShowNativeClasses;

				for (TObjectIterator<UReplicationGraph> It; It; ++It)
				{
					It->LogGraph(DebugInfo);
				}

				ensureAlwaysMsgf(false, TEXT("Very large replication list size requested. NewExpectedSize: %d"), NewExpectedSize);
			}
		};
	}
#endif

	// Remove handles to networked actors when their worlds are cleaned up
	WorldCleanupDelegateHandle = FWorldDelegates::OnWorldCleanup.AddWeakLambda(
		this,
		[this](UWorld* World, bool, bool)
		{
			// Build a list then iterate it to avoid removing while iterating
			TArray<AActor*> ActorsInWorld;
			ActorsInWorld.Reserve(ActiveNetworkActors.Num());

			for (AActor* Actor : ActiveNetworkActors)
			{
				if (Actor->GetWorld() == World)
				{
					ActorsInWorld.Emplace(Actor);
				}
			}

			for (AActor* Actor : ActorsInWorld)
			{
				RemoveNetworkActor(Actor);
			}
		});
}

void UReplicationGraph::BeginDestroy()
{
	FWorldDelegates::OnWorldCleanup.Remove(WorldCleanupDelegateHandle);
	WorldCleanupDelegateHandle.Reset();

	// Report any leaked actors
	// Must occur in BeginDestroy() because by FinishDestroy() the actors have been destroyed.
	if (!ActiveNetworkActors.IsEmpty())
	{
		UE_LOGF(LogReplicationGraph, Warning, "%d actors were not removed from ReplicationGraph", ActiveNetworkActors.Num());

		for (AActor* Actor : ActiveNetworkActors)
		{
			UE_LOGF(LogReplicationGraph, Log, "  Leaked actor: %ls", *GetFullNameSafe(Actor));
		}
	}

	Super::BeginDestroy();
}

extern void CountReplicationGraphSharedBytes_Private(FArchive& Ar);

void UReplicationGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraph::Serialize");

		// Currently, there is some global memory associated with RepGraph.
		// If there happens to be multiple RepGraphs, that would cause it to be counted multiple times.
		// This works, as "obj list" is the primary use case of counting memory, but it would break
		// if different legitimate memory counts happened in the same frame.
		static uint64 LastSharedCountFrame = 0;
		if (GFrameCounter != LastSharedCountFrame)
		{
			LastSharedCountFrame = GFrameCounter;

			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepGraphSharedBytes", CountReplicationGraphSharedBytes_Private(Ar));
		}

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PrioritizedReplicationList", PrioritizedReplicationList.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GlobalActorReplicationInfoMap", GlobalActorReplicationInfoMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ActiveNetworkActors", ActiveNetworkActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RPCSendPolicyMap", RPCSendPolicyMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RPC_Multicast_OpenChannelForClass", RPC_Multicast_OpenChannelForClass.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CSVTracker", CSVTracker.CountBytes(Ar));

		if (FastSharedReplicationBunch)
		{
			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("FastSharedReplicationBunch",
				Ar.CountBytes(sizeof(FOutBunch), sizeof(FOutBunch));
				FastSharedReplicationBunch->CountMemory(Ar);
			);
		}
	}
}

void UReplicationGraph::InitForNetDriver(UNetDriver* InNetDriver)
{
#if WITH_SERVER_CODE
	LLM_SCOPE_BYTAG(NetRepGraph);

	NetDriver = InNetDriver;

	InitGlobalActorClassSettings();
	InitGlobalGraphNodes();

	UE::Net::Private::LastTimeLoggedMoreInfo = 0;

	for (UNetConnection* ClientConnection : NetDriver->ClientConnections)
	{
		AddClientConnection(ClientConnection);
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraph::TearDown()
{
	CSVTracker.TearDown();

	Super::TearDown();
}

void UReplicationGraph::InitNode(UReplicationGraphNode* Node)
{
	Node->Initialize(GraphGlobals);

	if (Node->GetRequiresPrepareForReplication())
	{
		PrepareForReplicationNodes.Add(Node);
	}
}

void UReplicationGraph::InitGlobalActorClassSettings()
{
	// AInfo and APlayerControllers have no world location, so distance scaling should always be 0
	FClassReplicationInfo NonSpatialClassInfo;
	NonSpatialClassInfo.DistancePriorityScale = 0.f;

	GlobalActorReplicationInfoMap.SetClassInfo( AInfo::StaticClass(), NonSpatialClassInfo );
	GlobalActorReplicationInfoMap.SetClassInfo( APlayerController::StaticClass(), NonSpatialClassInfo );

	RPC_Multicast_OpenChannelForClass.Reset();
	RPC_Multicast_OpenChannelForClass.Set(AActor::StaticClass(), true); // Open channels for multicast RPCs by default
	RPC_Multicast_OpenChannelForClass.Set(AController::StaticClass(), false); // multicasts should never open channels on Controllers since opening a channel on a non-owner breaks the Controller's replication.
	RPC_Multicast_OpenChannelForClass.Set(AServerStatReplicator::StaticClass(), false);	
}

void UReplicationGraph::InitGlobalGraphNodes()
{
	// TODO: We should come up with a basic/default implementation for people to use to model
}

void UReplicationGraph::InitConnectionGraphNodes(UNetReplicationGraphConnection* ConnectionManager)
{
	// This handles tear off actors. Child classes should call Super::InitConnectionGraphNodes.
	ConnectionManager->TearOffNode = CreateNewNode<UReplicationGraphNode_TearOff_ForConnection>();
	ConnectionManager->AddConnectionGraphNode(ConnectionManager->TearOffNode);
}

void UReplicationGraph::AddGlobalGraphNode(UReplicationGraphNode* GraphNode)
{
	GlobalGraphNodes.Add(GraphNode);
}

void UReplicationGraph::RemoveGlobalGraphNode(UReplicationGraphNode* GraphNode)
{
	if(GraphNode)
	{
		GlobalGraphNodes.RemoveSwap(GraphNode);
		PrepareForReplicationNodes.RemoveSwap(GraphNode);
	}
}

void UReplicationGraph::AddConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetReplicationGraphConnection* ConnectionManager)
{
	ConnectionManager->AddConnectionGraphNode(GraphNode);
}

void UReplicationGraph::RemoveConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetReplicationGraphConnection* ConnectionManager)
{
	ConnectionManager->RemoveConnectionGraphNode(GraphNode);
}

UNetReplicationGraphConnection* UReplicationGraph::FindConnectionManager(UNetConnection* NetConnection) const
{
#if WITH_SERVER_CODE
	check(NetConnection);

	// Children do not have a connection manager, this is handled by their parent.
	// We do not want to create connection managers for children, so redirect them.
	if (NetConnection->GetUChildConnection() != nullptr)
	{
		NetConnection = ((UChildConnection*)NetConnection)->Parent;
		UE_LOGF(LogReplicationGraph, Verbose, "UReplicationGraph::FindOrAddConnectionManager was called with a child connection, redirecting to parent");
		check(NetConnection != nullptr);
	}

	check(NetConnection->GetDriver() == NetDriver);

	for (UNetReplicationGraphConnection* ConnManager : Connections)
	{
		if (ConnManager->NetConnection == NetConnection)
		{
			return ConnManager;
		}
	}

	for (UNetReplicationGraphConnection* ConnManager : PendingConnections)
	{
		if (ConnManager->NetConnection == NetConnection)
		{
			return ConnManager;
		}
	}
#endif

	return nullptr;
}

UNetReplicationGraphConnection* UReplicationGraph::FindOrAddConnectionManager(UNetConnection* NetConnection)
{
#if WITH_SERVER_CODE
	check(NetConnection);

	// Children do not have a connection manager, this is handled by their parent.
	// We do not want to create connection managers for children, so redirect them.
	if (NetConnection->GetUChildConnection() != nullptr)
	{
		NetConnection = ((UChildConnection*)NetConnection)->Parent;
		UE_LOGF(LogReplicationGraph, Verbose, "UReplicationGraph::FindOrAddConnectionManager was called with a child connection, redirecting to parent");
		check(NetConnection != nullptr);
	}

	check(NetConnection->GetDriver() == NetDriver);

	// Could use an acceleration map if necessary
	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_FindConnectionManager)
	for (UNetReplicationGraphConnection* ConnManager : Connections)
	{
		if (ConnManager->NetConnection == NetConnection)
		{
			return ConnManager;
		}
	}

	for (UNetReplicationGraphConnection* ConnManager : PendingConnections)
	{
		if (ConnManager->NetConnection == NetConnection)
		{
			return ConnManager;
		}
	}

	// We dont have one yet, create one but put it in the pending list. ::AddClientConnection *should* be called soon!
	UNetReplicationGraphConnection* NewManager = CreateClientConnectionManagerInternal(NetConnection);
	PendingConnections.Add(NewManager);
	return NewManager;
#else
	return nullptr;
#endif // WITH_SERVER_CODE
}

void UReplicationGraph::AddClientConnection(UNetConnection* NetConnection)
{
#if WITH_SERVER_CODE
	// Children do not have a connection manager, do not proceed with this function in this case.
	// Default behavior never calls this function with child connections anyways, so this is really only here for protection.
	if (NetConnection->GetUChildConnection() != nullptr)
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraph::AddClientConnection was called with a child connection, dropping.");
		return;
	}

	// We may have already created a manager for this connection in the pending list
	for (int32 i=PendingConnections.Num()-1; i >= 0; --i)
	{
		if (UNetReplicationGraphConnection* ConnManager = PendingConnections[i])
		{
			if (ConnManager->NetConnection == NetConnection)
			{
				PendingConnections.RemoveAtSwap(i, EAllowShrinking::No);
				Connections.Add(ConnManager);
				return;
			}
		}
	}

	// Create it
	Connections.Add(CreateClientConnectionManagerInternal(NetConnection));
#endif // WITH_SERVER_CODE
}

UNetReplicationGraphConnection* UReplicationGraph::CreateClientConnectionManagerInternal(UNetConnection* Connection)
{
	repCheckf(Connection->GetReplicationConnectionDriver() == nullptr, TEXT("Connection %s on NetDriver %s already has a ReplicationConnectionDriver %s"), *GetNameSafe(Connection), *GetNameSafe(Connection->Driver), *Connection->GetReplicationConnectionDriver()->GetName() );

	// Create the object
	UNetReplicationGraphConnection* NewConnectionManager = NewObject<UNetReplicationGraphConnection>(this, ReplicationConnectionManagerClass.Get());

	// Give it an ID
	const int32 NewConnectionNum = Connections.Num() + PendingConnections.Num();
	NewConnectionManager->ConnectionOrderNum = NewConnectionNum;

	// Initialize it with us
	NewConnectionManager->InitForGraph(this);

	// Associate NetConnection with it
	NewConnectionManager->InitForConnection(Connection);

	// Create Graph Nodes for this specific connection
	InitConnectionGraphNodes(NewConnectionManager);

	return NewConnectionManager;
}

UNetReplicationGraphConnection* UReplicationGraph::FixGraphConnectionList(TArray<UNetReplicationGraphConnection*>& OutList, int32& ConnectionNum, UNetConnection* RemovedNetConnection)
{
	UNetReplicationGraphConnection* RemovedGraphConnection(nullptr);

	for (int32 Index = 0; Index < OutList.Num(); ++Index)
	{
		UNetReplicationGraphConnection* CurrentGraphConnection = OutList[Index];
		if (CurrentGraphConnection->NetConnection != RemovedNetConnection)
		{
			// Fix the ConnectionOrderNum
			const int32 NewConnectionNum = ConnectionNum++;
			CurrentGraphConnection->ConnectionOrderNum = NewConnectionNum;
		}
		else
		{
			// Found the connection to remove
			ensureMsgf(RemovedGraphConnection==nullptr, TEXT("Found multiple GraphConnections for the same NetConnection: %s.  PreviousGraphConnection(%i): %s | CurrentGraphConnection(%i): %s"),
				*RemovedNetConnection->Describe(), 
				RemovedGraphConnection->ConnectionOrderNum, *RemovedGraphConnection->GetName(),
				CurrentGraphConnection->ConnectionOrderNum, *CurrentGraphConnection->GetName());
			RemovedGraphConnection = CurrentGraphConnection;

			OutList.RemoveAtSwap(Index--, EAllowShrinking::No);
		}
	}

	return RemovedGraphConnection;
}

void UReplicationGraph::RemoveClientConnection(UNetConnection* NetConnection)
{
	// Children do not have a connection manager, do not attempt to remove it here.
	// Default behavior never calls this function with child connections anyways, so this is really only here for protection.
	if (NetConnection->GetUChildConnection() != nullptr)
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraph::RemoveClientConnection was called with a child connection!");
		return;
	}

	int32 ConnectionNum = 0;

	UNetReplicationGraphConnection* ActiveGraphConnectionRemoved = FixGraphConnectionList(MutableView(Connections), ConnectionNum, NetConnection);
	UNetReplicationGraphConnection* PendingGraphConnectionRemoved = FixGraphConnectionList(MutableView(PendingConnections), ConnectionNum, NetConnection);

	if (ActiveGraphConnectionRemoved)
	{
		ActiveGraphConnectionRemoved->TearDown();
		ensure(PendingGraphConnectionRemoved == nullptr);
	}

	if (PendingGraphConnectionRemoved)
	{
		PendingGraphConnectionRemoved->TearDown();
		ensure(ActiveGraphConnectionRemoved == nullptr);
	}

	if (!ActiveGraphConnectionRemoved && !PendingGraphConnectionRemoved)
	{
		// At least one list should have found the connection
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraph::RemoveClientConnection could not find connection %ls in Connections (%d) or PendingConnections (%d) lists", *GetNameSafe(NetConnection), Connections.Num(), PendingConnections.Num());
	}
}

void UReplicationGraph::SetRepDriverWorld(UWorld* InWorld)
{
	if (GraphGlobals.IsValid())
	{
		GraphGlobals->World = InWorld;
	}
}

void UReplicationGraph::InitializeActorsInWorld(UWorld* InWorld)
{
#if WITH_SERVER_CODE
	check(GraphGlobals.IsValid());
	checkf(GraphGlobals->World == InWorld, TEXT("UReplicationGraph::InitializeActorsInWorld world mismatch. %s vs %s"), *GetPathNameSafe(GraphGlobals->World), *GetPathNameSafe(InWorld));

	if (InWorld)
	{
		if (InWorld->AreActorsInitialized())
		{
			InitializeForWorld(InWorld);
		}
		else
		{
			// World isn't initialized yet. This happens when launching into a map directly from command line
			InWorld->OnActorsInitialized.AddLambda([&](const UWorld::FActorsInitializedParams& P)
			{
				this->InitializeForWorld(P.World);
			});
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraph::InitializeForWorld(UWorld* World)
{
#if WITH_SERVER_CODE
	LLM_SCOPE_BYTAG(NetRepGraph);

	ActiveNetworkActors.Reset();
	GlobalActorReplicationInfoMap.ResetActorMap();

	for (UReplicationGraphNode* Manager : GlobalGraphNodes)
	{
		Manager->NotifyResetAllNetworkActors();
	}

	for (UNetReplicationGraphConnection* RepGraphConnection : Connections)
	{
		RepGraphConnection->NotifyResetAllNetworkActors();
	}
	
	if (World)
	{
		for (FActorIterator Iter(World); Iter; ++Iter)
		{
			AActor* Actor = *Iter;
			if (IsValid(Actor) && ULevel::IsNetActor(Actor))
			{
				AddNetworkActor(Actor);
			}
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraph::AddNetworkActor(AActor* Actor)
{
	LLM_SCOPE_BYTAG(NetRepGraph);
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_AddNetworkActor);

	if (IsActorValidForReplicationGather(Actor) == false)
	{
		return;
	}

	if (NetDriver && !NetDriver->ShouldReplicateActor(Actor))
	{
		return;
	}

	bool bWasAlreadyThere = false;
	ActiveNetworkActors.Add(Actor, &bWasAlreadyThere);
	if (bWasAlreadyThere)
	{
		// Guarding against double adds
		return;
	}

	ensureMsgf(!Actor->bNetTemporary, TEXT("ReplicationGraph does not support bNetTemporary. Actor: %s has bNetTemporary set."), *Actor->GetPathName());

	// Create global rep info	
	FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(Actor);
	GlobalInfo.bWantsToBeDormant = Actor->NetDormancy > DORM_Awake;

	RouteAddNetworkActorToNodes(FNewReplicatedActorInfo(Actor), GlobalInfo);
}

void UReplicationGraph::SetRoleSwapOnReplicate(AActor* Actor, bool bSwapRoles)
{
	if (FGlobalActorReplicationInfo* GlobalInfo = GlobalActorReplicationInfoMap.Find(Actor))
	{
		GlobalInfo->bSwapRolesOnReplicate = bSwapRoles;
	}
}

void UReplicationGraph::RouteAddNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo)
{
	// The base implementation just routes to every global node. Subclasses will want a more direct routing function where possible.
	for (UReplicationGraphNode* Node : GlobalGraphNodes)
	{
		Node->NotifyAddNetworkActor(ActorInfo);
	}
}

void UReplicationGraph::RemoveNetworkActor(AActor* Actor)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_RemoveNetworkActor);

	if (UNLIKELY(!IsValid(Actor)))
	{
		UE_LOGF(
			LogReplicationGraph,
			Warning,
			"Actor should still be valid during removal from networking: %ls",
			*GetFullNameSafe(Actor));

		if (!Actor)
		{
			return;
		}
	}

	const bool bWasInActiveSet = ActiveNetworkActors.Remove(Actor) > 0;
	if (UNLIKELY(!bWasInActiveSet))
	{
		UE_LOGF(
			LogReplicationGraph,
			Verbose,
			"Removing Actor that was not in the active set: %ls",
			*GetFullNameSafe(Actor));
	}
	else
	{
		// Tear off actors have already been removed from the nodes, so we don't need to route them again.
		if (Actor->GetTearOff() == false)
		{
			UE_CLOGF(
				CVar_RepGraph_LogActorRemove > 0,
				LogReplicationGraph,
				Display,
				"UReplicationGraph::RemoveNetworkActor %ls",
				*GetFullNameSafe(Actor));
			RouteRemoveNetworkActorToNodes(FNewReplicatedActorInfo(Actor));
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_RemoveNetworkActor_FromConnectionsMap);

			for (UNetReplicationGraphConnection* ConnectionManager : Connections)
			{
				ConnectionManager->ActorInfoMap.RemoveActor(Actor);
				ConnectionManager->RemoveActorFromAllPrevDormantActorLists(Actor);
			}
		}
	}

	const bool bWasInReplicationInfoMap = GlobalActorReplicationInfoMap.Remove(Actor) > 0;

	if (UNLIKELY(!bWasInActiveSet && bWasInReplicationInfoMap))
	{
		// Actor info found for an actor that was not in the active set, likely to crash if the actor is garbage collected while there is still a
		// record of it. Perhaps a replication node inadvertently created the record via GlobalActorReplicationInfoMap.Get().
		UE_LOGF(
			LogReplicationGraph,
			Warning,
			"Actor replication info found for actor that was not in the active set: %ls",
			*GetFullNameSafe(Actor));
	}
}

void UReplicationGraph::RouteRemoveNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_RouteRemoveNetworkActorToNodes);
	// The base implementation just routes to every global node. Subclasses will want a more direct routing function where possible.
	for (UReplicationGraphNode* Node : GlobalGraphNodes)
	{
		Node->NotifyRemoveNetworkActor(ActorInfo);
	}
}

void UReplicationGraph::RouteRenameNetworkActorToNodes(const FRenamedReplicatedActorInfo& ActorInfo)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_RouteRenameNetworkActorToNodes);
	
	// The base implementation just routes to every global node. Subclasses will want a more direct routing function where possible.
	for (UReplicationGraphNode* Node : GlobalGraphNodes)
	{
		Node->NotifyActorRenamed(ActorInfo);
	}
}

void UReplicationGraph::ForceNetUpdate(AActor* Actor)
{
	if (FGlobalActorReplicationInfo* RepInfo = GlobalActorReplicationInfoMap.Find(Actor))
	{
		RepInfo->ForceNetUpdateFrame = ReplicationGraphFrame;
#if REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE
		RepInfo->Events.ForceNetUpdate.Broadcast(Actor, *RepInfo);
#endif // REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE

		CSVTracker.PostActorForceUpdated(Actor->GetClass());
	}
}

void UReplicationGraph::FlushNetDormancy(AActor* Actor, bool bWasDormInitial)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_FlushNetDormancy);

	if (Actor->IsActorInitialized() == false)
	{
		UE_CLOGF(CVar_RepGraph_LogActorAdd > 0, LogReplicationGraph, Display, "UReplicationGraph::FlushNetDormancy called on %ls but not fully initiailized yet. Discarding.", *Actor->GetPathName());
		return;
	}

	if (IsActorValidForReplication(Actor) == false)
	{
		UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "UReplicationGraph::FlushNetDormancy called on %ls. Ignored since actor is destroyed or about to be", *Actor->GetPathName());
		return;
	}

	FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(Actor);
	const bool bNewWantsToBeDormant = (Actor->NetDormancy > DORM_Awake);

	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "UReplicationGraph::FlushNetDormancy %ls. Old WantsToBeDormant: %d. New WantsToBeDormant: %d", *Actor->GetPathName(), GlobalInfo.bWantsToBeDormant, bNewWantsToBeDormant);

	if (GlobalInfo.bWantsToBeDormant != bNewWantsToBeDormant)
	{
		UE_LOGF(LogReplicationGraph, Verbose, "UReplicationGraph::FlushNetDormancy %ls. WantsToBeDormant is changing (%d -> %d) from a Flush! We expect NotifyActorDormancyChange to be called first.", *Actor->GetPathName(), (bool)GlobalInfo.bWantsToBeDormant, bNewWantsToBeDormant);
		GlobalInfo.bWantsToBeDormant = Actor->NetDormancy > DORM_Awake;
	}

	if (GlobalInfo.bWantsToBeDormant == false)
	{
		// This actor doesn't want to be dormant. Suppress the Flush call into the nodes. This is to prevent wasted work since the AActor code calls NotifyActorDormancyChange then Flush always.
		return;
	}

	if (ReplicationGraphFrame == GlobalInfo.LastFlushNetDormancyFrame)
	{
		// We already did this work this frame, we can early out
		return;
	}

	GlobalInfo.LastFlushNetDormancyFrame = ReplicationGraphFrame;

	if (bWasDormInitial)
	{
		AddNetworkActor(Actor);
	}

	FNotifyActorFlushDormancy DormancyFlushEvent = GlobalInfo.Events.DormancyFlush;
	DormancyFlushEvent.Broadcast(Actor, GlobalInfo);

	// Stinks to have to iterate through like this, especially when net driver is doing a similar thing.
	// Dormancy should probably be rewritten.
	for (UNetReplicationGraphConnection* ConnectionManager: Connections)
	{
		ConnectionManager->SetActorNotDormantOnConnection(Actor);
	}
}

void UReplicationGraph::NotifyActorTearOff(AActor* Actor)
{
	// All connections that currently have a channel for the actor will put this actor on their TearOffNode.
	for (UNetReplicationGraphConnection* ConnectionManager: Connections)
	{
		if (FConnectionReplicationActorInfo* Info = ConnectionManager->ActorInfoMap.Find(Actor))
		{
			UActorChannel* Channel = Info->Channel;
			if (Channel && Channel->Actor)
			{
				Info->bTearOff = true; // Tells ServerReplicateActors to close the channel the next time this replicates
				ConnectionManager->TearOffNode->NotifyTearOffActor(Actor, Info->LastRepFrameNum); // Tells this connection to gather this actor (until it replicates again)
			}
		}
	}

	// Remove the actor from the rest of the graph. The tear off node will add it from here.
	RouteRemoveNetworkActorToNodes(FNewReplicatedActorInfo(Actor));
}

void UReplicationGraph::NotifyActorFullyDormantForConnection(AActor* Actor, UNetConnection* Connection)
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_NotifyActorFullyDormantForConnection);

	// Children do not have a connection manager, so redirect as necessary.
	// This is unlikely to be reached as child connections don't open their own channels
	if (Connection->GetUChildConnection() != nullptr)
	{
		Connection = ((UChildConnection*)Connection)->Parent;
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraph::NotifyActorFullyDormantForConnection was called for a child connection %ls on actor %ls", *Connection->GetName(), *Actor->GetName());
		check(Connection != nullptr);
	}

	// This is kind of bad but unavoidable. Possibly could use acceleration map (actor -> connections) but that would be a pain to maintain.
	for (UNetReplicationGraphConnection* ConnectionManager: Connections)
	{
		if (ConnectionManager->NetConnection == Connection)
		{			
			if (FConnectionReplicationActorInfo* Info = ConnectionManager->ActorInfoMap.Find(Actor))
			{
				Info->bDormantOnConnection = true;
			}
			break;
		}
	}
}

void UReplicationGraph::NotifyActorDormancyChange(AActor* Actor, ENetDormancy OldDormancyState)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_NotifyActorDormancyChange);

	FGlobalActorReplicationInfo* ActorRepInfo = GlobalActorReplicationInfoMap.Find(Actor);
	if (!ActorRepInfo)
	{
		UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "UReplicationGraph::NotifyActorDormancyChange %ls. Ignoring change since actor is not registered yet.", *Actor->GetPathName());
		return;
	}

	if (IsActorValidForReplication(Actor) == false)
	{
		UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "UReplicationGraph::NotifyActorDormancyChange %ls. Ignoring change since actor is destroyed or about to be.", *Actor->GetPathName());
		return;
	}

	ENetDormancy CurrentDormancy = Actor->NetDormancy;

	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "UReplicationGraph::NotifyActorDormancyChange %ls. Old WantsToBeDormant: %d. New WantsToBeDormant: %d", *Actor->GetPathName(), ActorRepInfo->bWantsToBeDormant, CurrentDormancy > DORM_Awake ? 1 : 0);

	const bool bOldWantsToBeDormant = OldDormancyState > DORM_Awake;
	const bool bNewWantsToBeDormant = CurrentDormancy > DORM_Awake;

	ActorRepInfo->bWantsToBeDormant = bNewWantsToBeDormant;
	ActorRepInfo->Events.DormancyChange.Broadcast(Actor, *ActorRepInfo, CurrentDormancy, OldDormancyState);

	// Is the actor coming out of dormancy via changing its dormancy state?
	if (!bNewWantsToBeDormant && bOldWantsToBeDormant)
	{
		// Since the actor will now be in a non dormant state, calls to FlushNetDormancy will be be suppressed.
		// So we need to clear the per-connection dormancy bool here, since the one in FlushNetDormancy won't do it.
		for (UNetReplicationGraphConnection* ConnectionManager: Connections)
		{
			ConnectionManager->SetActorNotDormantOnConnection(Actor);
		}
	}
}

void UReplicationGraph::NotifyActorRenamed(AActor* Actor, UObject* PreviousOuter, FName PreviousName)
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_NotifyActorRenamed);
	CSV_CUSTOM_STAT(ReplicationGraph, NumActorRenames, 1, ECsvCustomStatOp::Accumulate);

	if (!IsActorValidForReplication(Actor))
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraph::NotifyActorRenamed %ls. Actor invalid for replication.", *GetFullNameSafe(Actor));
		return;
	}

	const bool bIsActorDynamic = !Actor->IsFullNameStableForNetworking();

	if (UE::Net::Private::CVar_RepGraph_HandleDynamicActorRename && bIsActorDynamic)
	{
		ULevel* PreviousLevel = Cast<ULevel>(PreviousOuter);
		FName PreviousStreamingLevelName = (PreviousLevel && PreviousLevel->IsPersistentLevel() == false) ? PreviousLevel->GetOutermost()->GetFName() : NAME_None;

		// Fix up dependencies in global list
		GlobalActorReplicationInfoMap.NotifyActorRenamed(Actor, PreviousStreamingLevelName);

		FRenamedReplicatedActorInfo RenameInfo(Actor, PreviousStreamingLevelName);
		RouteRenameNetworkActorToNodes(RenameInfo);
	}
}

FORCEINLINE bool ReadyForNextReplication(FConnectionReplicationActorInfo& ConnectionData, FGlobalActorReplicationInfo& GlobalData, const uint32 FrameNum)
{
	return (ConnectionData.NextReplicationFrameNum <= FrameNum || GlobalData.ForceNetUpdateFrame > ConnectionData.LastRepFrameNum);
}

FORCEINLINE bool ReadyForNextReplication_FastPath(FConnectionReplicationActorInfo& ConnectionData, FGlobalActorReplicationInfo& GlobalData, const uint32 FrameNum)
{
	return (ConnectionData.FastPath_NextReplicationFrameNum <= FrameNum || GlobalData.ForceNetUpdateFrame > ConnectionData.FastPath_LastRepFrameNum);
}

// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
//	Server Replicate Actors
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------

FNativeClassAccumulator ChangeClassAccumulator;
FNativeClassAccumulator NoChangeClassAccumulator;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool bTrackClassReplication = false;
#else
	const bool bTrackClassReplication = false;
#endif

int32 UReplicationGraph::ServerReplicateActors(float DeltaSeconds)
{
	LLM_SCOPE_BYTAG(NetRepGraph);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVar_RepGraph_Pause)
	{
		return 0;
	}

	// Frequency throttling: intended for testing and PIE special case
	int32 TargetUpdatesPerSecond = CVar_RepGraph_Frequency;  // Explicit override for testing
#if WITH_EDITOR
	if ( CVar_RepGraph_Frequency <= 0 && CVar_RepGraph_Frequency_MatchTargetInPIE > 0)
	{
		if (GIsEditor && GIsPlayInEditorWorld)
		{
			// When PIE, use target server tick rate. This is not perfect but will be closer than letting rep graph tick every frame.
			TargetUpdatesPerSecond = NetDriver->GetNetServerMaxTickRate();
		}
	}
#endif
	const float TimeBetweenUpdates = TargetUpdatesPerSecond > 0 ? (1.f / (float)TargetUpdatesPerSecond) : 0.f;

	TimeLeftUntilUpdate -= DeltaSeconds;
	if (TimeLeftUntilUpdate > 0.f)
	{
		return 0;
	}
	TimeLeftUntilUpdate = TimeBetweenUpdates;
#endif
	
	SCOPED_NAMED_EVENT(UReplicationGraph_ServerReplicateActors, FColor::Green);

	++NetDriver->ReplicationFrame;	// This counter is used by RepLayout to utilize CL/serialization sharing. We must increment it ourselves, but other places can increment it too, in order to invalidate the shared state.
	const uint32 FrameNum = ReplicationGraphFrame; // This counter is used internally and drives all frame based replication logic.
	FrameReplicationStats.Reset();

	bWasConnectionSaturated = false;
	bWasConnectionFastPathSaturated = false;

	TSet<UNetConnection*> ConnectionsToClose;

	ON_SCOPE_EXIT
	{
		// We increment this after our replication has happened. If we increment at the beginning of this function, then we rep with FrameNum X, then start the next game frame with the same FrameNum X. If at the top of that frame,
		// when processing packets, ticking, etc, we get calls to TearOff, ForceNetUpdate etc which make use of ReplicationGraphFrame, they will be using a stale frame num. So we could replicate, get a server move next frame, ForceNetUpdate, but think we 
		// already replicated this frame.
		ReplicationGraphFrame++;

		for (UNetConnection* ConnectionToClose : ConnectionsToClose)
		{
			ConnectionToClose->Close();
		}
	};

#if WITH_SERVER_CODE
	// -------------------------------------------------------
	//	PREPARE (Global)
	// -------------------------------------------------------

	{
		QUICK_SCOPE_CYCLE_COUNTER(NET_PrepareReplication);

		for (UReplicationGraphNode* Node : PrepareForReplicationNodes)
		{
			Node->PrepareForReplication();
		}
	}

	// -------------------------------------------------------
	// For Each Connection
	// -------------------------------------------------------
	
	// Total number of children processed, added to all the connections later for stat tracking purposes.
	int32 NumChildrenConnectionsProcessed = 0;

	// Wrap around connection selector
	HeavyComputationConnectionSelector = (HeavyComputationConnectionSelector + 1) % Connections.Num();

	for (UNetReplicationGraphConnection* ConnectionManager: Connections)
	{
		// Prepare for Replication also handles children as well.
		if (ConnectionManager->PrepareForReplication() == false)
		{
			// Connection is not ready to replicate
			continue;
		}

		FNetViewerArray ConnectionViewers;
		UNetConnection* const NetConnection = ConnectionManager->NetConnection;
		APlayerController* const PC = NetConnection->PlayerController;
		FPerConnectionActorInfoMap& ConnectionActorInfoMap = ConnectionManager->ActorInfoMap;
		
		repCheckf(NetConnection->GetReplicationConnectionDriver() == ConnectionManager, TEXT("NetConnection %s mismatch rep driver. %s vs %s"), *GetNameSafe(NetConnection), *GetNameSafe(NetConnection->GetReplicationConnectionDriver()), *GetNameSafe(ConnectionManager));

		const bool bReplayConnection = NetConnection->IsReplay();

		if (bReplayConnection && !NetConnection->IsReplayReady())
		{
			// replay isn't ready to record right now
			continue;
		}

		CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(ReplayNetConnection, bReplayConnection);

		ConnectionViewers.Emplace(NetConnection, 0.f);
		
		// Send ClientAdjustments (movement RPCs) do this first and never let bandwidth saturation suppress these.
		if (PC)
		{
			PC->SendClientAdjustment();
		}

		// Do the above but on all splitscreen connections as well.
		for (int32 ChildIdx = 0; ChildIdx < NetConnection->Children.Num(); ++ChildIdx)
		{
			UNetConnection* ChildConnection = NetConnection->Children[ChildIdx];
			if (ChildConnection && ChildConnection->PlayerController && ChildConnection->ViewTarget)
			{
				ChildConnection->PlayerController->SendClientAdjustment();

				ConnectionViewers.Emplace(ChildConnection, 0.f);
			}
		}

		NumChildrenConnectionsProcessed += NetConnection->Children.Num();

		// treat all other non-replay connections as viewers
		if (bReplayConnection)
		{
			for (UNetReplicationGraphConnection* RepGraphConn : Connections)
			{
				if ((RepGraphConn->NetConnection != NetConnection) && !RepGraphConn->NetConnection->IsReplay() && RepGraphConn->PrepareForReplication())
				{
					ConnectionViewers.Emplace(RepGraphConn->NetConnection, 0.f);
				}
			}

			AddReplayViewers(NetConnection, ConnectionViewers);
		}

		ON_SCOPE_EXIT
		{
			NetConnection->TrackReplicationForAnalytics(bWasConnectionSaturated);
			bWasConnectionSaturated = false;

			ConnectionManager->TrackReplicationForAnalytics(bWasConnectionFastPathSaturated);
			bWasConnectionFastPathSaturated = false;
		};

		const FReplicationGraphDestructionSettings DestructionSettings(DestructInfoMaxDistanceSquared, CVar_RepGraph_OutOfRangeDistanceCheckRatio * DestructInfoMaxDistanceSquared);

		ConnectionManager->QueuedBitsForActorDiscovery = 0;

		// --------------------------------------------------------------------------------------------------------------
		// GATHER list of ReplicationLists for this connection
		// --------------------------------------------------------------------------------------------------------------
		
		FGatheredReplicationActorLists GatheredReplicationListsForConnection;

		const bool bIsSelectedForHeavyComputation =
			HeavyComputationConnectionSelector == ConnectionManager->ConnectionOrderNum
			|| CVar_RepGraph_ConnectionHeavyComputationAmortization == 0;

		const FConnectionGatherActorListParameters Parameters(
			ConnectionViewers,
			*ConnectionManager,
			ConnectionManager->GetCachedClientVisibleLevelNames(),
			FrameNum,
			GatheredReplicationListsForConnection,
			bIsSelectedForHeavyComputation);

		UNetReplicationGraphConnection::FRepGraphDestructionViewerInfoArray DestructionViewersInfo;

		{
			QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_GatherForConnection);

			for (UReplicationGraphNode* Node : GlobalGraphNodes)
			{
				Node->GatherActorListsForConnection(Parameters);
			}

			for (UReplicationGraphNode* Node : ConnectionManager->ConnectionGraphNodes)
			{
				Node->GatherActorListsForConnection(Parameters);
			}

			ConnectionManager->UpdateGatherLocationsForConnection(ConnectionViewers, DestructionSettings);

			if (GatheredReplicationListsForConnection.NumLists() == 0)
			{
				UE_LOGF(LogReplicationGraph, Verbose, "No Replication Lists were returned for connection");
				continue;
			}

			for( const FNetViewer& NetViewer : ConnectionViewers )
			{
				FLastLocationGatherInfo* LastInfoForViewer = ConnectionManager->LastGatherLocations.FindByKey<UNetConnection*>(NetViewer.Connection);
				check(LastInfoForViewer);

				DestructionViewersInfo.Emplace(UNetReplicationGraphConnection::FRepGraphDestructionViewerInfo(NetViewer.ViewLocation, LastInfoForViewer->LastOutOfRangeLocationCheck));
			}
		}

		// --------------------------------------------------------------------------------------------------------------
		// PROCESS gathered replication lists
		// --------------------------------------------------------------------------------------------------------------
		{
			QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ProcessGatheredLists);

			ReplicateActorListsForConnections_Default(ConnectionManager, GatheredReplicationListsForConnection, ConnectionViewers);
			ReplicateActorListsForConnections_FastShared(ConnectionManager, GatheredReplicationListsForConnection, ConnectionViewers);
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_PostProcessGatheredLists);

			// ------------------------------------------
			// Handle stale, no longer relevant, actor channels.
			// ------------------------------------------			
			{
				QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_LookForNonRelevantChannels);

				for (auto MapIt = ConnectionActorInfoMap.CreateChannelIterator(); MapIt; ++MapIt)
				{
					FConnectionReplicationActorInfo& ConnectionActorInfo = *MapIt.Value().Get();
					UActorChannel* Channel = MapIt.Key();
					checkSlow(Channel != nullptr);
					checkSlow(ConnectionActorInfo.Channel != nullptr);

					// We check for Channel closing early and bail.
					// It may be possible when using Dormancy that an Actor's Channel was closed, but a new channel was created
					// before the original Cleaned Up.
					if (Channel->Closing)
					{
						UE_LOGF(LogReplicationGraph, Verbose, "NET_ReplicateActors_LookForNonRelevantChannels (key) Channel %ls is closing. Skipping.", *Channel->Describe());
						continue;
					}
					else if (ConnectionActorInfo.Channel->Closing)
					{
						UE_LOGF(LogReplicationGraph, Verbose, "NET_ReplicateActors_LookForNonRelevantChannels (value) Channel %ls is closing. Skipping.", *ConnectionActorInfo.Channel->Describe());
						continue;
					}

					ensureMsgf(Channel == ConnectionActorInfo.Channel, TEXT("Channel: %s ConnectionActorInfo.Channel: %s."), *Channel->Describe(), *ConnectionActorInfo.Channel->Describe());

					if (ConnectionActorInfo.ActorChannelCloseFrameNum > 0 && ConnectionActorInfo.ActorChannelCloseFrameNum <= FrameNum)
					{
						AActor* Actor = Channel->Actor;

						if (ensureMsgf(Actor,
							TEXT("Stale Connection Actor Info with Valid Channel but Invalid Actor. RelevantTime=%f, LastUpdateTime=%f, LastRepFrameNum=%d, RepPeriod=%d, CloseFrame=%d, CurrentRepFrame=%d, bTearOff=%d, bDormant=%d, Channel=%s, State=%d"),
							Channel->RelevantTime, Channel->LastUpdateTime, ConnectionActorInfo.LastRepFrameNum, ConnectionActorInfo.ReplicationPeriodFrame, ConnectionActorInfo.ActorChannelCloseFrameNum,
							FrameNum, !!ConnectionActorInfo.bTearOff, !!ConnectionActorInfo.bDormantOnConnection, *(Channel->Describe()), static_cast<int32>(NetConnection->GetConnectionState())))
						{
							if (Actor->IsNetStartupActor())
								continue;

							UpdateActorConnectionCounter(Actor, Channel->Connection, ActorChannelDestroyCounter);

							//UE_CLOGF(DebugConnection, LogReplicationGraph, Display, "Closing Actor Channel:0x%x 0x%X0x%X, %ls %d <= %d", ConnectionActorInfo.Channel, Actor, NetConnection, *GetNameSafe(ConnectionActorInfo.Channel->Actor), ConnectionActorInfo.ActorChannelCloseFrameNum, FrameNum);
							if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
							{
								UE_LOGF(LogReplicationGraph, Display, "Closing Actor Channel due to timeout: %ls. %d <= %d (%ls)", *(ConnectionActorInfo.Channel->Describe()), ConnectionActorInfo.ActorChannelCloseFrameNum, FrameNum, *NetConnection->Describe());
							}

							INC_DWORD_STAT_BY( STAT_NetActorChannelsClosed, 1 );
							ConnectionActorInfo.Channel->Close(EChannelCloseReason::Relevancy);

							// Make sure that we remove this actor from the PrevDormantActorList since it won't be dormant anymore since it will be destroyed
							ConnectionManager->SetActorNotDormantOnConnection(Actor);
						}						
					}
				}
			}

			// ------------------------------------------
			// Handle Destruction Infos. These are actors that have been destroyed on the server but that we need to tell the client about.
			// ------------------------------------------
			{
				QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateDestructionInfos);
				ConnectionManager->ReplicateDestructionInfos(DestructionViewersInfo, DestructionSettings);
			}

			// ------------------------------------------
			// Handle Dormant Destruction Infos. These are actors that are dormant but no longer relevant to the client.
			// ------------------------------------------
			{
				QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateDormantDestructionInfos);
				ConnectionManager->ReplicateDormantDestructionInfos();
			}

#if DO_ENABLE_REPGRAPH_DEBUG_ACTOR
			{
				RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateDebugActor);
				if (AReplicationGraphDebugActor* DebugActor = ConnectionManager->DebugActor.Get())
				{
					FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(DebugActor);
					FConnectionReplicationActorInfo& ActorInfo = ConnectionActorInfoMap.FindOrAdd(DebugActor);

					if (ReadyForNextReplication(ActorInfo, GlobalInfo, FrameNum))
					{
						const int32 DebugActorBits = IntCastChecked<int32>(ReplicateSingleActor(DebugActor, ActorInfo, GlobalInfo, ConnectionActorInfoMap, *ConnectionManager, FrameNum));
						// Do not count the debug actor towards our bandwidth limit
						NetConnection->QueuedBits -= DebugActorBits;
					}
				}
			}
#endif
		}

		if (NetConnection->GetPendingCloseDueToReplicationFailure())
		{
			ConnectionsToClose.Add(NetConnection);
		}
	}
	
	SET_DWORD_STAT(STAT_NumProcessedConnections, Connections.Num() + NumChildrenConnectionsProcessed);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVar_RepGraph_PrintTrackClassReplication)
	{
		CVar_RepGraph_PrintTrackClassReplication = 0;
		UE_LOGF(LogReplicationGraph, Display, "Changed Classes: %ls", *ChangeClassAccumulator.BuildString());
		UE_LOGF(LogReplicationGraph, Display, "No Change Classes: %ls", *NoChangeClassAccumulator.BuildString());
	}
#endif

	FrameReplicationStats.NumConnections = Connections.Num() + NumChildrenConnectionsProcessed;
	PostServerReplicateStats(FrameReplicationStats);

	CSVTracker.EndReplicationFrame();
#endif // WITH_SERVER_CODE
	return 0;
}

void UReplicationGraph::ReplicateActorListsForConnections_Default(UNetReplicationGraphConnection* ConnectionManager, FGatheredReplicationActorLists& GatheredReplicationListsForConnection, FNetViewerArray& Viewers)
{
#if WITH_SERVER_CODE
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const bool bEnableFullActorPrioritizationDetails = DO_REPGRAPH_DETAILS(bEnableFullActorPrioritizationDetailsAllConnections || ConnectionManager->bEnableFullActorPrioritizationDetails);
	const bool bDoDistanceCull = (CVar_RepGraph_SkipDistanceCull == 0);
	const bool bDoCulledOnConnectionCount = (CVar_RepGraph_PrintCulledOnConnectionClasses == 1);
	bTrackClassReplication = (CVar_RepGraph_TrackClassReplication > 0 || CVar_RepGraph_PrintTrackClassReplication > 0);
	if (!bTrackClassReplication)
	{
		ChangeClassAccumulator.Reset();
		NoChangeClassAccumulator.Reset();
	}

#else
	const bool bEnableFullActorPrioritizationDetails = false;
	const bool bDoDistanceCull = true;
	const bool bDoCulledOnConnectionCount = false;
#endif

	// Debug accumulators
	FNativeClassAccumulator DormancyClassAccumulator;
	FNativeClassAccumulator DistanceClassAccumulator;

	int32 NumGatheredActorsOnConnection = 0;
	int32 NumPrioritizedActorsOnConnection = 0;

	UNetConnection* const NetConnection = ConnectionManager->NetConnection;
	FPerConnectionActorInfoMap& ConnectionActorInfoMap = ConnectionManager->ActorInfoMap;
	const uint32 FrameNum = ReplicationGraphFrame;

	// --------------------------------------------------------------------------------------------------------------
	// PRIORITIZE Gathered Actors For Connection
	// --------------------------------------------------------------------------------------------------------------
	{
		QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_PrioritizeForConnection);

		// We will make a prioritized list for each item in the packet budget. (Each item may accept multiple categories. Each list has one category)
		// This means, depending on the packet budget, a gathered list could end up in multiple prioritized lists. This would not be desirable in most cases but is not explicitly forbidden.

		PrioritizedReplicationList.Reset();
		TArray<FPrioritizedRepList::FItem>* SortingArray = &PrioritizedReplicationList.Items;

		const float MaxDistanceScaling = PrioritizationConstants.MaxDistanceScaling;
		const uint32 MaxFramesSinceLastRep = PrioritizationConstants.MaxFramesSinceLastRep;

		// Add actors from gathered list
		const TArrayView<const FActorRepListType> Actors = GatheredReplicationListsForConnection.ViewActors(EActorRepListTypeFlags::Default);
		NumGatheredActorsOnConnection += Actors.Num();

		for (const FActorRepListType& Actor : Actors)
		{
			RG_QUICK_SCOPE_CYCLE_COUNTER(Prioritize_InnerLoop);

			if (!ensureMsgf(IsActorValidForReplication_LogMoreInfo(Actor), TEXT("Actor not valid for replication")))
			{
				continue;
			}

			// -----------------------------------------------------------------------------------------------------------------
			//	Prioritize Actor for Connection: this is the main block of code for calculating a final score for this actor
			//		-This is still pretty rough. It would be nice if this was customizable per project without suffering virtual calls.
			// -----------------------------------------------------------------------------------------------------------------

			if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
			{
				UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph PrioritizeActor: %ls", *Actor->GetName());
			}

			FConnectionReplicationActorInfo& ConnectionData = ConnectionActorInfoMap.FindOrAdd(Actor);

			RG_QUICK_SCOPE_CYCLE_COUNTER(Prioritize_InnerLoop_ConnGlobalLookUp);

			// Skip if dormant on this connection. We want this to always be the first/quickest check.
			if (ConnectionData.bDormantOnConnection)
			{
				DO_REPGRAPH_DETAILS(PrioritizedReplicationList.GetNextSkippedDebugDetails(Actor)->bWasDormant = true);
				if (bDoCulledOnConnectionCount)
				{
					DormancyClassAccumulator.Increment(Actor->GetClass());
				}
				continue;
			}

			FGlobalActorReplicationInfo& GlobalData = GlobalActorReplicationInfoMap.Get(Actor);

			RG_QUICK_SCOPE_CYCLE_COUNTER(Prioritize_InnerLoop_PostGlobalLookUp);

			// Skip if its not time to replicate on this connection yet. We have to look at ForceNetUpdateFrame here. It would be possible to clear
			// NextReplicationFrameNum on all connections when ForceNetUpdate is called. This probably means more work overall per frame though. Something to consider.
			if (!ReadyForNextReplication(ConnectionData, GlobalData, FrameNum))
			{
				DO_REPGRAPH_DETAILS(PrioritizedReplicationList.GetNextSkippedDebugDetails(Actor)->FramesTillNextReplication = (FrameNum - ConnectionData.LastRepFrameNum));
				continue;
			}

			RG_QUICK_SCOPE_CYCLE_COUNTER(Prioritize_InnerLoop_PostReady);

			// Output record for full debugging. This is not used in the actual sorting/prioritization of the list, just for logging/debugging purposes
			FPrioritizedActorFullDebugDetails* DebugDetails = nullptr;
			if (DO_REPGRAPH_DETAILS(UNLIKELY(bEnableFullActorPrioritizationDetails)))
			{
				DO_REPGRAPH_DETAILS(DebugDetails = PrioritizedReplicationList.GetNextFullDebugDetails(Actor));
			}

			float AccumulatedPriority = GlobalData.Settings.AccumulatedNetPriorityBias;

			// -------------------
			// Distance Scaling
			// -------------------
			if (GlobalData.Settings.DistancePriorityScale > 0.f)
			{
				// Always compute distance even for AlwaysRelevant actors since the priority scaling needs it.
				FVector::FReal SmallestDistanceSq = std::numeric_limits<FVector::FReal>::max();
				int32 ViewersThatSkipActor = 0;
				
				for (const FNetViewer& CurViewer : Viewers)
				{
					const FVector::FReal DistSq = (GlobalData.WorldLocation - CurViewer.ViewLocation).SizeSquared();
					SmallestDistanceSq = FMath::Min(DistSq, SmallestDistanceSq);

					// Figure out if we should be skipping this actor
					if (bDoDistanceCull && ConnectionData.GetCullDistanceSquared() > 0.f && DistSq > ConnectionData.GetCullDistanceSquared())
					{
						++ViewersThatSkipActor;
						continue;
					}
				}

				// If no one is near this actor, skip it.
				if (ViewersThatSkipActor >= Viewers.Num())
				{
					DO_REPGRAPH_DETAILS(PrioritizedReplicationList.GetNextSkippedDebugDetails(Actor)->DistanceCulled = static_cast<float>(FMath::Sqrt(SmallestDistanceSq)));

					// Skipped actors should not have any 
					if (bDoCulledOnConnectionCount)
					{
						DistanceClassAccumulator.Increment(Actor->GetClass());
					}
					continue;
				}

				const float DistanceFactor = FMath::Clamp(static_cast<float>(SmallestDistanceSq / MaxDistanceScaling), 0.f, 1.f) * GlobalData.Settings.DistancePriorityScale;
				if (DO_REPGRAPH_DETAILS(UNLIKELY(DebugDetails)))
				{
					DebugDetails->DistanceSq = SmallestDistanceSq;
					DebugDetails->DistanceFactor = DistanceFactor;
				}

				AccumulatedPriority += DistanceFactor;
			}

			RG_QUICK_SCOPE_CYCLE_COUNTER(Prioritize_InnerLoop_PostCull);

			// Update the timeout frame number here. (Since this was returned by the graph, regardless if we end up replicating or not, we bump up the timeout frame num. This has to be done here because Distance Scaling can cull the actor
			UpdateActorChannelCloseFrameNum(Actor, ConnectionData, GlobalData, FrameNum, NetConnection);

			//UE_CLOGF(DebugConnection, LogReplicationGraph, Display, "0x%X0x%X ConnectionData.ActorChannelCloseFrameNum=%d on %d", Actor, NetConnection, ConnectionData.ActorChannelCloseFrameNum, FrameNum);

			// -------------------
			// Starvation Scaling
			// -------------------
			if (GlobalData.Settings.StarvationPriorityScale > 0.f)
			{
				// StarvationPriorityScale = scale "Frames since last rep". E.g, 2.0 means treat every missed frame as if it were 2, etc.
				const float FramesSinceLastRep = ((float)(FrameNum - ConnectionData.LastRepFrameNum)) * GlobalData.Settings.StarvationPriorityScale;
				const float StarvationFactor = 1.f - FMath::Clamp<float>(FramesSinceLastRep / (float)MaxFramesSinceLastRep, 0.f, 1.f);

				AccumulatedPriority += StarvationFactor;

				if (DO_REPGRAPH_DETAILS(UNLIKELY(DebugDetails)))
				{
					DebugDetails->FramesSinceLastRap = static_cast<uint32>(FMath::TruncToInt32(FramesSinceLastRep));
					DebugDetails->StarvationFactor = StarvationFactor;
				}
			}

			// ------------------------
			// Pending dormancy scaling
			// ------------------------

			// Make sure pending dormant actors that have replicated at least once are prioritized,
			// so we actually mark them dormant quickly, skip future work, and close their channels.
			// Otherwise, newly spawned or never-replicated actors may starve out existing actors trying to go dormant.
			if (GlobalData.bWantsToBeDormant && ConnectionData.LastRepFrameNum > 0)
			{
				AccumulatedPriority -= 1.5f;
			}

			// -------------------
			//	Game code priority
			// -------------------

			if (GlobalData.ForceNetUpdateFrame > ConnectionData.LastRepFrameNum)
			{
				// Note that in legacy ForceNetUpdate did not actually bump priority. This gives us a hard coded bump if we haven't replicated since the last ForceNetUpdate frame.
				AccumulatedPriority -= 1.f;

				if (DO_REPGRAPH_DETAILS(UNLIKELY(DebugDetails)))
				{
					DebugDetails->GameCodeScaling = -1.f;
				}
			}
			
			// -------------------
			// Always prioritize the connection's owner and view target, since these are the most important actors for the client.
			// -------------------
			for (const FNetViewer& CurViewer : Viewers)
			{
				// We need to find if this is anyone's viewer or viewtarget, not just the parent connection.
				if (Actor == CurViewer.ViewTarget || Actor == CurViewer.InViewer)
				{
					if (UE::Net::CVar_ForceConnectionViewerPriority > 0)
					{
						AccumulatedPriority = -MAX_FLT;
					}
					else
					{
						AccumulatedPriority -= 10.0f;
					}
					break;
				}
			}

			SortingArray->Emplace(FPrioritizedRepList::FItem(AccumulatedPriority, Actor, &GlobalData, &ConnectionData));
		}

		{
			// Sort the merged priority list. We could potentially move this into the replicate loop below, this could potentially save use from sorting arrays that don't fit into the budget
			RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_PrioritizeForConnection_Sort);
			NumPrioritizedActorsOnConnection += SortingArray->Num();
			SortingArray->Sort();
		}
	}
	
	{
		QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateActorsForConnection);
		ReplicateActorsForConnection(NetConnection, ConnectionActorInfoMap, ConnectionManager, FrameNum);
	}


	// Broadcast the list we just handled. This is intended to be for debugging/logging features.
	ConnectionManager->OnPostReplicatePrioritizeLists.Broadcast(ConnectionManager, &PrioritizedReplicationList);

	if (bDoCulledOnConnectionCount)
	{
		UE_LOGF(LogReplicationGraph, Display, "Dormant Culled classes: %ls", *DormancyClassAccumulator.BuildString());
		UE_LOGF(LogReplicationGraph, Display, "Dist Culled classes: %ls", *DistanceClassAccumulator.BuildString());
		UE_LOGF(LogReplicationGraph, Display, "Saturated Connections: %d", GNumSaturatedConnections);
		UE_LOGF(LogReplicationGraph, Display, "");

		UE_LOGF(LogReplicationGraph, Display, "Gathered Actors: %d, PrioritizedActors: %d", NumGatheredActorsOnConnection, NumPrioritizedActorsOnConnection);
		UE_LOGF(LogReplicationGraph, Display, "Connection Loaded Streaming Levels: %d", NetConnection->ClientVisibleLevelNames.Num());
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraph::ReplicateActorsForConnection(UNetConnection* NetConnection, FPerConnectionActorInfoMap& ConnectionActorInfoMap, UNetReplicationGraphConnection* ConnectionManager, const uint32 FrameNum)
{
#if WITH_SERVER_CODE
	QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateActorsForConnection);

	for (int32 ActorIdx = 0; ActorIdx < PrioritizedReplicationList.Items.Num(); ++ActorIdx)
	{
		const FPrioritizedRepList::FItem& RepItem = PrioritizedReplicationList.Items[ActorIdx];

		AActor* Actor = RepItem.Actor;
		FConnectionReplicationActorInfo& ActorInfo = *RepItem.ConnectionData;

		// Always skip if we've already replicated this frame. This happens if an actor is in more than one replication list
		if (ActorInfo.LastRepFrameNum == FrameNum)
		{
			INC_DWORD_STAT_BY(STAT_NetRepActorListDupes, 1);
			continue;
		}

		FGlobalActorReplicationInfo& GlobalActorInfo = *RepItem.GlobalData;

		int64 BitsWritten = ReplicateSingleActor(Actor, ActorInfo, GlobalActorInfo, ConnectionActorInfoMap, *ConnectionManager, FrameNum);

		// --------------------------------------------------
		//	Update Packet Budget Tracking
		// --------------------------------------------------

		if (IsConnectionReady(NetConnection) == false)
		{
			// We've exceeded the budget for this category of replication list.
			RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_PartialStarvedActorList);
			HandleStarvedActorList(*ConnectionManager, PrioritizedReplicationList, ActorIdx + 1, ConnectionActorInfoMap, FrameNum);
			NotifyConnectionSaturated(*ConnectionManager);
			break;
		}
	}
#endif // WITH_SERVER_CODE
}

struct FScopedQueuedBits
{
	FScopedQueuedBits(int32& InQueuedBits, int32& InTotalBits) : QueuedBits(InQueuedBits), TotalBits(InTotalBits) { }
	~FScopedQueuedBits() { QueuedBits -= TotalBits; }
	int32& QueuedBits;
	int32& TotalBits;
};

// Tracks total bits/cpu and pushes to the CSV profiler

struct FScopedFastPathTracker
{
	FScopedFastPathTracker(UClass* InActorClass, FReplicationGraphCSVTracker& InTracker, int32& InBitsWritten) 
#if CSV_PROFILER_STATS
		: ActorClass(InActorClass), Tracker(InTracker), BitsWritten(InBitsWritten)
#endif
	{

#if CSV_PROFILER_STATS
#if STATS
		bEnabled = true;
#else
		bEnabled = FCsvProfiler::Get()->IsCapturing();
#endif
		if (bEnabled)
		{
			StartTime = FPlatformTime::Seconds();
		}
#endif
	}

#if CSV_PROFILER_STATS
	~FScopedFastPathTracker()
	{
		if (bEnabled)
		{
			const double FinalTime = FPlatformTime::Seconds() - StartTime;
			Tracker.PostFastPathReplication(ActorClass, FinalTime, BitsWritten);
		}
	}

	UClass* ActorClass;
	FReplicationGraphCSVTracker& Tracker;
	double StartTime;
	int32& BitsWritten;
	bool bEnabled;
#endif
};

void UReplicationGraph::ReplicateActorListsForConnections_FastShared(UNetReplicationGraphConnection* ConnectionManager, FGatheredReplicationActorLists& GatheredReplicationListsForConnection, FNetViewerArray& Viewers)
{
#if WITH_SERVER_CODE
	if (CVar_RepGraph_EnableFastSharedPath == false)
	{
		return;
	}

	if (GatheredReplicationListsForConnection.ContainsLists(EActorRepListTypeFlags::FastShared) == false)
	{
		return;
	}

	FPerConnectionActorInfoMap& ConnectionActorInfoMap = ConnectionManager->ActorInfoMap;
	UNetConnection* const NetConnection = ConnectionManager->NetConnection;
	const uint32 FrameNum = ReplicationGraphFrame;
	const float FastSharedDistanceRequirementPct = FastSharedPathConstants.DistanceRequirementPct;
	const int64 MaxBits = FastSharedPathConstants.MaxBitsPerFrame;
	const int32 StartIdx = FrameNum * FastSharedPathConstants.ListSkipPerFrame;

	int32 TotalBitsWritten = 0;

	// Fast shared path "doesn't count" towards our normal net send rate. This will subtract the bits we send in this function out of the queued bits on net connection.
	// This really isn't ideal. We want to have better ways of tracking and limiting network traffic. This feels pretty hacky in implementation but conceptually is good.
	FScopedQueuedBits ScopedQueuedBits(NetConnection->QueuedBits, TotalBitsWritten);

	const TArrayView<const FActorRepListType> Actors = GatheredReplicationListsForConnection.ViewActors(EActorRepListTypeFlags::FastShared);
	for (int32 i = 0; i < Actors.Num(); ++i)
	{
		// Round robin through the list over multiple frames. We want to avoid sorting this list based on 'time since last rep'. This is a good balance
		AActor* Actor = Actors[(StartIdx + i) % Actors.Num()];

		int32 BitsWritten = 0;

		if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
		{
			UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph FastShared Path Replication: %ls", *Actor->GetName());
		}

		FConnectionReplicationActorInfo& ConnectionData = ConnectionActorInfoMap.FindOrAdd(Actor);

		// Don't fast path rep if we already repped in the default path this frame
		if (UNLIKELY(ConnectionData.LastRepFrameNum == FrameNum))
		{
			continue;
		}

		if (UNLIKELY(ConnectionData.bTearOff))
		{
			continue;
		}

		// Actor channel must already be established to rep fast path
		UActorChannel* ActorChannel = ConnectionData.Channel;
		if (ActorChannel == nullptr || ActorChannel->Closing)
		{
			continue;
		}

		FGlobalActorReplicationInfo& GlobalActorInfo = GlobalActorReplicationInfoMap.Get(Actor);
		if (GlobalActorInfo.Settings.FastSharedReplicationFunc == nullptr)
		{
			// This actor does not support fastshared replication
			// FIXME: we should avoid this by keeping these actors on separate lists
			continue;
		}

		// Determine if this actor has any view relevancy to any connection this client has
		bool bNoViewRelevency = true;
		for (const FNetViewer& CurView : Viewers)
		{
			const FVector& ConnectionViewLocation = CurView.ViewLocation;
			const FVector& ConnectionViewDir = CurView.ViewDir;

			// Simple dot product rejection: only fast rep actors in front of this connection
			const FVector DirToActor = GlobalActorInfo.WorldLocation - ConnectionViewLocation;
			if (!(FVector::DotProduct(DirToActor, ConnectionViewDir) < 0.f))
			{
				bNoViewRelevency = false;
				break;
			}

			// Simple distance cull
			const FVector::FReal DistSq = DirToActor.SizeSquared();
			if (!(DistSq > (ConnectionData.GetCullDistanceSquared() * FastSharedDistanceRequirementPct)))
			{
				bNoViewRelevency = false;
				break;
			}
		}

		// Skip out if they have none.
		if (bNoViewRelevency)
		{
			continue;
		}

		BitsWritten = (int32)ReplicateSingleActor_FastShared(Actor, ConnectionData, GlobalActorInfo, *ConnectionManager, FrameNum);
		TotalBitsWritten += BitsWritten;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		static bool SkipCheck = false;
		if (SkipCheck)
		{
			continue;
		}
#endif
		if (TotalBitsWritten > MaxBits)
		{
			NotifyConnectionFastPathSaturated();
			return;
		}
	}
#endif // WITH_SERVER_CODE
}

REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.FastShared.ForceFull", CVar_RepGraph_FastShared_ForceFull, 0, "Redirects calls to ReplicateSingleActor_FastShared to ReplicateSingleActor");

int64 UReplicationGraph::ReplicateSingleActor_FastShared(AActor* Actor, FConnectionReplicationActorInfo& ConnectionData, FGlobalActorReplicationInfo& GlobalActorInfo, UNetReplicationGraphConnection& ConnectionManager, const uint32 FrameNum)
{
#if WITH_SERVER_CODE
	UNetConnection* NetConnection = ConnectionManager.NetConnection;

	// No matter what we consider this FastShared rep to happen. Even if the actor doesn't produce a bunch or its empty or stale, etc. We still consider this replication to have happened
	// for high level frequency purposes (E.g, UReplicationGraphNode_DynamicSpatialFrequency). But we want to do the update at the end of this function, not at the top since it can early out
	// if the actor doesn't produce a new bunch and this connection already got the last bunch produced.
	ON_SCOPE_EXIT
	{
		ConnectionData.FastPath_LastRepFrameNum = FrameNum;
		ConnectionData.FastPath_NextReplicationFrameNum = FrameNum + ConnectionData.FastPath_ReplicationPeriodFrame;
	};

	TOptional<FScopedActorRoleSwap> SwapGuard;
	if (GlobalActorInfo.bSwapRolesOnReplicate)
	{
		SwapGuard = FScopedActorRoleSwap(Actor);
	}

	// always replicate full pawns to the replay
	if (NetConnection->IsReplay() || CVar_RepGraph_FastShared_ForceFull > 0)
	{
		return ReplicateSingleActor(Actor, ConnectionData, GlobalActorInfo, ConnectionManager.ActorInfoMap, ConnectionManager, FrameNum);
	}

	int32 BitsWritten = 0;
	FScopedFastPathTracker ScopedTracker(Actor->GetClass(), CSVTracker, BitsWritten);	// Track time and bandwidth for this class	

	UActorChannel* ActorChannel = ConnectionData.Channel;

	// Actor channel must already be established to rep fast path
	if (ActorChannel == nullptr || ActorChannel->Closing)
	{
		return 0;
	}

	FrameReplicationStats.NumReplicatedFastPathActors++;

	// Allocate the shared bunch if it hasn't been already
	if (GlobalActorInfo.FastSharedReplicationInfo.IsValid() == false)
	{
		GlobalActorInfo.FastSharedReplicationInfo = MakeUnique<FFastSharedReplicationInfo>();
	}
	FOutBunch& OutBunch = GlobalActorInfo.FastSharedReplicationInfo->Bunch;

	// Update the shared bunch if its out of date
	if (GlobalActorInfo.FastSharedReplicationInfo->LastAttemptBuildFrameNum < FrameNum)
	{
		GlobalActorInfo.FastSharedReplicationInfo->LastAttemptBuildFrameNum = FrameNum;

		if (GlobalActorInfo.Settings.FastSharedReplicationFunc == nullptr)
		{
#if !(UE_BUILD_SHIPPING)
			static TSet<FObjectKey> WarnedClasses;
			if (WarnedClasses.Contains(FObjectKey(Actor->GetClass())) == false)
			{
				WarnedClasses.Add(FObjectKey(Actor->GetClass()));
				UE_LOGF(LogReplicationGraph, Warning, "::ReplicateSingleActor_FastShared called on %ls (%ls) when it doesn't have a FastSharedReplicationFunc defined, skipping actor. This is ineffecient.", *GetPathNameSafe(Actor), *Actor->GetClass()->GetName());
			}
#endif
			return 0;
		}

		// Make shared thing
		FastSharedReplicationBunch = &OutBunch;
		FastSharedReplicationChannel = ActorChannel;
		FastSharedReplicationFuncName = GlobalActorInfo.Settings.FastSharedReplicationFuncName;

		// Calling this function *should* result in an RPC call that we trap and fill out FastSharedReplicationBunch. See UReplicationGraph::ProcessRemoteFunction
		if (GlobalActorInfo.Settings.FastSharedReplicationFunc(Actor) == false)
		{
			// Something failed and we don't want to fast replicate. We wont check again this frame
			FastSharedReplicationBunch = nullptr;
			FastSharedReplicationChannel = nullptr;
			FastSharedReplicationFuncName = NAME_None;
			return 0;
		}

		
		if (FastSharedReplicationBunch == nullptr)
		{
			// A new bunch was produced this frame. (FastSharedReplicationBunch is cleared in ::ProcessRemoteFunction)
			GlobalActorInfo.FastSharedReplicationInfo->LastBunchBuildFrameNum = FrameNum;
		}
		else
		{
			// A new bunch was not produced this frame, but there is still valid data (If FastSharedReplicationFunc returns false, there is no valid data)
			FastSharedReplicationBunch = nullptr;
			FastSharedReplicationChannel = nullptr;
			FastSharedReplicationFuncName = NAME_None;
		}
	}

	if (ConnectionData.FastPath_LastRepFrameNum >= GlobalActorInfo.FastSharedReplicationInfo->LastBunchBuildFrameNum)
	{
		// We already repped this bunch to this connection. So just return
		return 0;
	}

	if (OutBunch.GetNumBits() <= 0)
	{
		// Empty bunch - no need to send. This means we aren't fast repping this actor this frame
		return 0;
	}

	// Setup the connection specifics on the bunch before calling SendBunch
	OutBunch.ChName = ActorChannel->ChName;
	OutBunch.ChIndex = ActorChannel->ChIndex;
	OutBunch.Channel = ActorChannel;
	OutBunch.Next = nullptr;

	// SendIt
	{
		FGuardValue_Bitfield(ActorChannel->bHoldQueuedExportBunchesAndGUIDs, true);
		
		ActorChannel->SendBunch(&OutBunch, false);
	}

	ensureAlwaysMsgf(OutBunch.bHasMustBeMappedGUIDs == 0, TEXT("FastShared bHasMustBeMappedGUIDs! %s"), *Actor->GetPathName());

	// Need to reset flags set on bunch send as we are re-using the bunch for more than one connection
	OutBunch.bPartial = 0;
	OutBunch.bPartialInitial = 0;
	OutBunch.bPartialFinal = 0;

	return OutBunch.GetNumBits();
#else
	return 0;
#endif // WITH_SERVER_CODE
}

int64 UReplicationGraph::ReplicateSingleActor(const FActorRepListType& Actor, FConnectionReplicationActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalActorInfo, FPerConnectionActorInfoMap& ConnectionActorInfoMap, UNetReplicationGraphConnection& ConnectionManager, const uint32 FrameNum)
{
#if WITH_SERVER_CODE
	RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_ReplicateSingleActor);

	// These checks will happen anyway in UActorChannel::ReplicateActor, but we need to be able to detect them to prevent crashes.
	// We could consider removing the actor from RepGraph if we hit these cases, but we don't have a good way to notify
	// game code or the Net Driver.
	if (!ensureMsgf(Actor, TEXT("Null Actor! Channel = %s"), *DescribeSafe(ActorInfo.Channel)))
	{
		return 0;
	}

	UNetConnection* NetConnection = ConnectionManager.NetConnection;

	if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph::ReplicateSingleActor: %ls. NetConnection: %ls", *Actor->GetName(), *NetConnection->Describe());
	}

	if (!ensureMsgf(IsActorValidForReplication_LogMoreInfo(Actor), TEXT("Actor not valid for replication")))
	{
		return 0;
	}

	FReplicationGraphCSVTracker::EActorFlags CSVFlags = FReplicationGraphCSVTracker::EActorFlags::None;

	if (LIKELY(ActorInfo.Channel))
	{
		if (UNLIKELY(ActorInfo.Channel->Closing))
		{
			// We are waiting for the client to ack this actor channel's close bunch.
			return 0;
		}
		else if (!ensureMsgf(ActorInfo.Channel->Actor == Actor, TEXT("Mismatched channel actors! Channel = %s, Replicating Actor = %s"), *ActorInfo.Channel->Describe(), *Actor->GetFullName()))
		{
			return 0;
		}

		// If this actor has passed the repgraph last-rep-frame check for normal replication,
		// repgraph thinks the actor has not been replicated to this connection yet this frame.
		// However, if the actor was replicated due to an RPC opening a new channel, repgraph is unaware
		// and will attempt to replicate it here.
#if REPGRAPH_CSV_TRACKER
		if (ActorInfo.Channel->LastUpdateTime == NetDriver->GetElapsedTime())
		{
			CSVFlags |= FReplicationGraphCSVTracker::EActorFlags::AlreadyReplicatedThisFrame;
		}
#endif
	}

	FrameReplicationStats.NumReplicatedActors++;

	ActorInfo.LastRepFrameNum = FrameNum;
	ActorInfo.NextReplicationFrameNum = FrameNum + ActorInfo.ReplicationPeriodFrame;

	UClass* const ActorClass = Actor->GetClass();

	/** Call PreReplication if necessary. */
	if (GlobalActorInfo.LastPreReplicationFrame != FrameNum)
	{
		RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_CallPreReplication);
		GlobalActorInfo.LastPreReplicationFrame = FrameNum;

		Actor->CallPreReplication(NetDriver);
	}

	TOptional<FScopedActorRoleSwap> SwapGuard;
	if (GlobalActorInfo.bSwapRolesOnReplicate)
	{
		SwapGuard = FScopedActorRoleSwap(Actor);
	}

	const bool bWantsToGoDormant = GlobalActorInfo.bWantsToBeDormant;

	bool bOpenActorChannel = (ActorInfo.Channel == nullptr);

	if (bOpenActorChannel)
	{
		// Create a new channel for this actor.
		INC_DWORD_STAT_BY( STAT_NetActorChannelsOpened, 1 );
		ActorInfo.Channel = (UActorChannel*)NetConnection->CreateChannelByName( NAME_Actor, EChannelCreateFlags::OpenedLocally );
		if ( !ActorInfo.Channel )
		{
			return 0;
		}

		CSVTracker.PostActorChannelCreated(ActorClass);

		//UE_LOGF(LogReplicationGraph, Display, "Created Actor Channel:0x%x 0x%X0x%X, %d", ActorInfo.Channel, Actor, NetConnection, FrameNum);
					
		// This will unfortunately cause a callback to this  UNetReplicationGraphConnection and will relook up the ActorInfoMap and set the channel that we already have set.
		// This is currently unavoidable because channels are created from different code paths (some outside of this loop)
		ActorInfo.Channel->SetChannelActor(Actor, ESetChannelActorFlags::None);
	}

	if (UNLIKELY(bWantsToGoDormant))
	{
		ActorInfo.Channel->StartBecomingDormant();
	}

	int64 BitsWritten = 0;
	const double StartingReplicateActorTimeSeconds = GReplicateActorTimeSeconds;

	if (UNLIKELY(ActorInfo.bTearOff))
	{
		// Replicate and immediately close in tear off case
		BitsWritten = ActorInfo.Channel->ReplicateActor();
		BitsWritten += ActorInfo.Channel->Close(EChannelCloseReason::TearOff);
	}
	else
	{
		// Just replicate normally
		BitsWritten = ActorInfo.Channel->ReplicateActor();
	}

	const double DeltaReplicateActorTimeSeconds = GReplicateActorTimeSeconds - StartingReplicateActorTimeSeconds;
	const bool bWasDataSent = BitsWritten > 0;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bTrackClassReplication)
	{
		if (bWasDataSent)
		{
			ChangeClassAccumulator.Increment(ActorClass);
		}
		else
		{
			NoChangeClassAccumulator.Increment(ActorClass);
		}
	}
#endif

	if (bWasDataSent == false)
	{
		FrameReplicationStats.NumReplicatedCleanActors++;
	}

	const bool bIsTrafficActorDiscovery = ActorDiscoveryMaxBitsPerFrame > 0 && (ActorInfo.Channel && ActorInfo.Channel->SpawnAcked == false);
	const bool bIsActorDiscoveryBudgetFull = bIsTrafficActorDiscovery && (ConnectionManager.QueuedBitsForActorDiscovery >= ActorDiscoveryMaxBitsPerFrame);

	if (bIsTrafficActorDiscovery && !bIsActorDiscoveryBudgetFull)
	{
		CSVFlags |= FReplicationGraphCSVTracker::EActorFlags::IsInDiscovery;
	}

	CSVTracker.PostReplicateActor(ActorClass, DeltaReplicateActorTimeSeconds, BitsWritten, CSVFlags);

	// ----------------------------
	//	Dependent actors
	// ----------------------------
	FGatheredReplicationActorLists ListGatherer;
	GlobalActorInfo.GatherDependentActorLists(ConnectionManager, ListGatherer);
	
	if (ListGatherer.NumLists() > 0)
	{
		RG_QUICK_SCOPE_CYCLE_COUNTER(NET_ReplicateActors_DependentActors);

		const int32 CloseFrameNum = ActorInfo.ActorChannelCloseFrameNum;
		
		const TArrayView<const FActorRepListType> DependentActors = ListGatherer.ViewActors(EActorRepListTypeFlags::Default);

		for (AActor* DependentActor : DependentActors)
		{
			repCheck(DependentActor);

			FConnectionReplicationActorInfo& DependentActorConnectionInfo = ConnectionActorInfoMap.FindOrAdd(DependentActor);
			FGlobalActorReplicationInfo& DependentActorGlobalData = GlobalActorReplicationInfoMap.Get(DependentActor);

			UpdateActorChannelCloseFrameNum(DependentActor, DependentActorConnectionInfo, DependentActorGlobalData, FrameNum, NetConnection);

			// Dependent actor channel will stay open as long as the owning actor channel is open
			DependentActorConnectionInfo.ActorChannelCloseFrameNum = FMath::Max<uint32>(CloseFrameNum, DependentActorConnectionInfo.ActorChannelCloseFrameNum);

			if (!ReadyForNextReplication(DependentActorConnectionInfo, DependentActorGlobalData, FrameNum))
			{
				continue;
			}

			if (!ensureMsgf(IsActorValidForReplication(DependentActor), TEXT("DependentActor %s (Owner: %s) not valid for replication (BeingDestroyed:%d) (IsValid:%d) (Unreachable:%d) (TearOff:%d)! Channel = %s"),
				*DependentActor->GetFullName(), *Actor->GetFullName(),
				DependentActor->IsActorBeingDestroyed(), IsValid(DependentActor), DependentActor->IsUnreachable(), DependentActor->GetTearOff(),
				*DescribeSafe(DependentActorConnectionInfo.Channel)))
			{
				continue;
			}

			//UE_LOGF(LogReplicationGraph, Display, "DependentActor %ls %ls. NextReplicationFrameNum: %d. FrameNum: %d. ForceNetUpdateFrame: %d. LastRepFrameNum: %d.", *DependentActor->GetPathName(), *NetConnection->GetName(), DependentActorConnectionInfo.NextReplicationFrameNum, FrameNum, DependentActorGlobalData.ForceNetUpdateFrame, DependentActorConnectionInfo.LastRepFrameNum);
			BitsWritten += ReplicateSingleActor(DependentActor, DependentActorConnectionInfo, DependentActorGlobalData, ConnectionActorInfoMap, ConnectionManager, FrameNum);
		}
	}

	// Optional budget for actor discovery traffic
	if (bIsTrafficActorDiscovery && !bIsActorDiscoveryBudgetFull)
	{
		const int32 DiscoveryBits = IntCastChecked<int32>(BitsWritten);

		ConnectionManager.QueuedBitsForActorDiscovery += DiscoveryBits;

		// Remove the discovery traffic from the regular traffic
		NetConnection->QueuedBits -= DiscoveryBits;
		BitsWritten = 0;
	}

	return BitsWritten;
#else
	return 0;
#endif //WITH_SERVER_CODE
}

void UReplicationGraph::HandleStarvedActorList(const UNetReplicationGraphConnection& RepGraphConnection, const FPrioritizedRepList& List, int32 StartIdx, FPerConnectionActorInfoMap& ConnectionActorInfoMap, uint32 FrameNum)
{
	FGatheredReplicationActorLists DependentListContainer;

	for (int32 ActorIdx=StartIdx; ActorIdx < List.Items.Num(); ++ActorIdx)
	{
		const FPrioritizedRepList::FItem& RepItem = List.Items[ActorIdx];
		const uint32 CloseFrameNum = RepItem.ConnectionData->ActorChannelCloseFrameNum;

		// Update dependent actor's timeout frame
		FGlobalActorReplicationInfo& GlobalActorInfo = GlobalActorReplicationInfoMap.Get(RepItem.Actor);

		
		GlobalActorInfo.GatherDependentActorLists(RepGraphConnection, DependentListContainer);
		const TArrayView<const FActorRepListType> DependentActors = DependentListContainer.ViewActors(EActorRepListTypeFlags::Default);
		for (AActor* DependentActor : DependentActors)
		{
			FConnectionReplicationActorInfo& DependentActorConnectionInfo = ConnectionActorInfoMap.FindOrAdd(DependentActor);
			DependentActorConnectionInfo.ActorChannelCloseFrameNum = FMath::Max<uint32>(CloseFrameNum, DependentActorConnectionInfo.ActorChannelCloseFrameNum);
		}
		DependentListContainer.Reset();
	}
}

void UReplicationGraph::UpdateActorChannelCloseFrameNum(AActor* Actor, FConnectionReplicationActorInfo& ConnectionData, const FGlobalActorReplicationInfo& GlobalData, const uint32 FrameNum, UNetConnection* NetConnection) const
{
	if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph::UpdateActorChannelCloseFrameNum: %ls. Channel: %ls FrameNum: %d ActorChannelFrameTimeout: %d.", *Actor->GetName(), *(ConnectionData.Channel ? ConnectionData.Channel->Describe() : FString(TEXT("None"))), FrameNum, GlobalData.Settings.ActorChannelFrameTimeout);
	}

	// Only update if the actor has a timeout set
	if (GlobalData.Settings.ActorChannelFrameTimeout > 0)
	{
		const uint32 NewCloseFrameNum = FrameNum + ConnectionData.ReplicationPeriodFrame + GlobalData.Settings.ActorChannelFrameTimeout + GlobalActorChannelFrameNumTimeout;
		ConnectionData.ActorChannelCloseFrameNum = FMath::Max<uint32>(ConnectionData.ActorChannelCloseFrameNum, NewCloseFrameNum); // Never go backwards, something else could have bumped it up further intentionally
	}
}

bool UReplicationGraph::ProcessRemoteFunction(class AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, class UObject* SubObject )
{
#if WITH_SERVER_CODE
	// ----------------------------------
	// Setup
	// ----------------------------------

	if (RepGraphConditionalActorBreakpoint(Actor, nullptr))
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph::ProcessRemoteFunction: %ls. Function: %ls.", *GetNameSafe(Actor), *GetNameSafe(Function));
	}

	if (IsActorValidForReplication(Actor) == false || Actor->IsActorBeingDestroyed())
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraph::ProcessRemoteFunction: Actor %ls destroyed or not ready! Function: %ls.", *GetNameSafe(Actor), *GetNameSafe(Function));
		return true;
	}

	// get the top most function
	while( Function->GetSuperFunction() )
	{
		Function = Function->GetSuperFunction();
	}

	// If we have a subobject, thats who we are actually calling this on. If no subobject, we are calling on the actor.
	UObject* TargetObj = SubObject ? SubObject : Actor;

	// Make sure this function exists for both parties.
	const FClassNetCache* ClassCache = NetDriver->NetCache->GetClassNetCache( TargetObj->GetClass() );
	if (!ClassCache)
	{
		UE_LOGF(LogReplicationGraph, Warning, "ClassNetCache empty, not calling %ls::%ls", *Actor->GetName(), *Function->GetName());
		return true;
	}
		
	const FFieldNetCache* FieldCache = ClassCache->GetFromField( Function );
	if ( !FieldCache )
	{
		UE_LOGF(LogReplicationGraph, Warning, "FieldCache empty, not calling %ls::%ls", *Actor->GetName(), *Function->GetName());
		return true;
	}


	// --------------------------------------------------------------------------------------------------------------------------------------------------------------------------
	// FastShared Replication. This is ugly but the idea here is to just fill out the bunch parameters and return so that this bunch can be reused by other connections
	// --------------------------------------------------------------------------------------------------------------------------------------------------------------------------
	if (FastSharedReplicationBunch && (FastSharedReplicationFuncName == Function->GetFName()))
	{
		// We also cache off a channel so we can call some of the serialization functions on it. This isn't really necessary though and we could break those parts off
		// into a static function.
		if (ensureMsgf(FastSharedReplicationChannel, TEXT("FastSharedReplicationPath set but FastSharedReplicationChannel is not! %s"), *Actor->GetPathName()))
		{
			// Reset the bunch here. It will be reused and we should only reset it right before we actually write to it.
			FastSharedReplicationBunch->Reset();
			FastSharedReplicationBunch->NetTokensPendingExport.Reset();

			// It sucks we have to a temp writer like this, but we don't know how big the payload will be until we serialize it
			FNetBitWriter TempWriter(nullptr, 0);

#if UE_NET_TRACE_ENABLED
			// Create trace collector if tracing is enabled for the target bunch
			FNetTraceCollector* Collector = GetTraceCollector(*FastSharedReplicationBunch);
			if (Collector)
			{
				Collector->Reset();
			}
			else
			{
				Collector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);
				SetTraceCollector(*FastSharedReplicationBunch, Collector);
			}

			// We use the collector from the shared bunch
			SetTraceCollector(TempWriter, Collector);
#endif // UE_NET_TRACE_ENABLED

			// Create export scope, storing NetTokens pending export in FastSharedReplicationBunch->NetTokensPendingExport
			UE::Net::FNetTokenExportScope NetTokenExportScope(TempWriter, NetDriver->GetNetTokenStore(), FastSharedReplicationBunch->NetTokensPendingExport, "SendPropertiesForRPC");

			TSharedPtr<FRepLayout> RepLayout = NetDriver->GetFunctionRepLayout( Function );
			RepLayout->SendPropertiesForRPC(Function, FastSharedReplicationChannel, TempWriter, Parameters);

			FNetBitWriter TempBlockWriter(nullptr, 0);

#if UE_NET_TRACE_ENABLED
			// Ugliness to get data reported correctly, we basically fold the data from TempWriter into TempBlockWriter and then to Bunch
			// Create trace collector if tracing is enabled for the target bunch			
			SetTraceCollector(TempBlockWriter, Collector ? UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace) : nullptr);
#endif // UE_NET_TRACE_ENABLED

			FastSharedReplicationChannel->WriteFieldHeaderAndPayload( TempBlockWriter, ClassCache, FieldCache, nullptr, TempWriter, true );

#if UE_NET_TRACE_ENABLED
			// As we have used the collector we need to reset it before injecting the final data
			if (Collector)
			{
				Collector->Reset();
			}
			UE_NET_TRACE_OBJECT_SCOPE(FastSharedReplicationChannel->ActorNetGUID, *FastSharedReplicationBunch, Collector, ENetTraceVerbosity::Trace);
#endif
			FastSharedReplicationChannel->WriteContentBlockPayload( TargetObj, *FastSharedReplicationBunch, false, TempBlockWriter );
			
			// Release temporary collector
			UE_NET_TRACE_DESTROY_COLLECTOR(GetTraceCollector(TempBlockWriter));

			FastSharedReplicationBunch = nullptr;
			FastSharedReplicationChannel = nullptr;
			FastSharedReplicationFuncName = NAME_None;
		}
		return true;
	}


	// ----------------------------------
	// Multicast
	// ----------------------------------

	if ((Function->FunctionFlags & FUNC_NetMulticast))
	{
		TSharedPtr<FRepLayout> RepLayout = NetDriver->GetFunctionRepLayout( Function );

		TOptional<FVector> ActorLocation;

		UNetDriver::ERemoteFunctionSendPolicy SendPolicy = UNetDriver::ERemoteFunctionSendPolicy::Default;
		if (CVar_RepGraph_EnableRPCSendPolicy > 0)
		{
			if (FRPCSendPolicyInfo* FuncSendPolicy = RPCSendPolicyMap.Find(FObjectKey(Function)))
			{
				if (FuncSendPolicy->bSendImmediately)
				{
					SendPolicy = UNetDriver::ERemoteFunctionSendPolicy::ForceSend;
				}
			}
		}

		RepLayout->BuildSharedSerializationForRPC(Parameters, NetDriver->GetNetTokenStore());
		FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(Actor);

		bool ForceFlushNetDormancy = false;

		// Cache streaming level name off
		FNewReplicatedActorInfo NewActorInfo(Actor);
		const FName ActorStreamingLevelName = NewActorInfo.StreamingLevelName;
		EProcessRemoteFunctionFlags RemoteFunctionFlags = EProcessRemoteFunctionFlags::None;
		
		for (UNetReplicationGraphConnection* Manager : Connections)
		{
			FConnectionReplicationActorInfo& ConnectionActorInfo = Manager->ActorInfoMap.FindOrAdd(Actor);
			UNetConnection* NetConnection = Manager->NetConnection;

			if (NetConnection->IsClosingOrClosed())
			{
				return true;
			}

			// This connection isn't ready yet
			if (NetConnection->ViewTarget == nullptr)
			{
				continue;
			}

			// Streaming level actor that the client doesn't have loaded. Do not send.
			if (ActorStreamingLevelName != NAME_None && NetConnection->ClientVisibleLevelNames.Contains(ActorStreamingLevelName) == false)
			{
				continue;
			}

			// While the channel is closing we cannot send new multicast RPCs
			if (ConnectionActorInfo.Channel && ConnectionActorInfo.Channel->Closing)
			{
				continue;
			}
			
			//UE_CLOGF(ConnectionActorInfo.Channel == nullptr, LogReplicationGraph, Display, "Null channel on %ls for %ls", *GetPathNameSafe(Actor), *GetNameSafe(Function));
			if (ConnectionActorInfo.Channel == nullptr && (RPC_Multicast_OpenChannelForClass.GetChecked(Actor->GetClass()) == true))
			{
				// There is no actor channel here. Ideally we would just ignore this but in the case of net dormancy, this may be an actor that will replicate on the next frame.
				// If the actor is dormant and is a distance culled actor, we can probably safely assume this connection will open a channel for the actor on the next rep frame.
				// This isn't perfect and we may want a per-function or per-actor policy that allows to dictate what happens in this situation.

				// Actors being destroyed (Building hit with rocket) will wake up before this gets hit. So dormancy really cant be relied on here.
				// if (Actor->NetDormancy > DORM_Awake)
				{
					bool bShouldOpenChannel = true;
					if (ConnectionActorInfo.GetCullDistanceSquared() > 0.f)
					{
						bShouldOpenChannel = false;
						if (ActorLocation.IsSet() == false)
						{
							ActorLocation = Actor->GetActorLocation();
						}

						FNetViewerArray ViewsToConsider;
						ViewsToConsider.Emplace(NetConnection, 0.f);

						for (int32 ChildIdx = 0; ChildIdx < NetConnection->Children.Num(); ++ChildIdx)
						{
							if (NetConnection->Children[ChildIdx]->ViewTarget != nullptr)
							{
								ViewsToConsider.Emplace(NetConnection->Children[ChildIdx], 0.f);
							}
						}

						// Loop through and see if we should keep this channel open, as when we do distance, we will
						// default to the channel being closed.
						for (const FNetViewer& Viewer : ViewsToConsider)
						{
							const FVector::FReal DistSq = (ActorLocation.GetValue() - Viewer.ViewLocation).SizeSquared();
							if (DistSq <= ConnectionActorInfo.GetCullDistanceSquared())
							{
								bShouldOpenChannel = true;
								break;
							}
						}
					}

					if (bShouldOpenChannel)
					{
#if !UE_BUILD_SHIPPING
						if (Actor->bOnlyRelevantToOwner && (!Actor->GetNetOwner() || (Actor->GetNetOwner() != NetConnection->PlayerController)))
						{
							UE_LOGF(LogReplicationGraph, Warning, "Multicast RPC opening channel for bOnlyRelevantToOwner actor, check RPC_Multicast_OpenChannelForClass: Actor: %ls Target: %ls Function: %ls", *GetNameSafe(Actor), *GetNameSafe(TargetObj), *GetNameSafe(Function));
							ensureMsgf(Cast<APlayerController>(Actor) == nullptr, TEXT("MulticastRPC %s will open a channel for %s to a non-owner. This will break the PlayerController replication."), *Function->GetName(), *GetNameSafe(Actor));
						}
#endif

						// We are within range, we will open a channel now for this actor and call the RPC on it
						ConnectionActorInfo.Channel = (UActorChannel*)NetConnection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally);

						if (ConnectionActorInfo.Channel)
						{
							ConnectionActorInfo.Channel->SetChannelActor(Actor, ESetChannelActorFlags::None);

							// Update timeout frame name. We would run into problems if we open the channel, queue a bunch, and then it timeouts before RepGraph replicates properties.
							UpdateActorChannelCloseFrameNum(Actor, ConnectionActorInfo, GlobalInfo, ReplicationGraphFrame+1 /** Plus one to error on safe side. RepFrame num will be incremented in the next tick */, NetConnection );

							// If this actor is dormant on the connection, we will force a flushnetdormancy call.
							ForceFlushNetDormancy |= ConnectionActorInfo.bDormantOnConnection;
						}
					}
				}
			}
			
			if (ConnectionActorInfo.Channel)
			{
				NetDriver->ProcessRemoteFunctionForChannel(ConnectionActorInfo.Channel, ClassCache, FieldCache, TargetObj, NetConnection, Function, Parameters, OutParms, Stack, true, SendPolicy, RemoteFunctionFlags);

				if (SendPolicy == UNetDriver::ERemoteFunctionSendPolicy::ForceSend)
				{
					// Queue the send in an array that we consume in PostTickDispatch to avoid force flushing multiple times a frame on the same connection
					ConnectionsNeedingsPostTickDispatchFlush.AddUnique(NetConnection);
				}

			}
		}

		RepLayout->ClearSharedSerializationForRPC();

		if (ForceFlushNetDormancy)
		{
			Actor->FlushNetDormancy();
		}
		return true;
	}

	// ----------------------------------
	// Single Connection
	// ----------------------------------
	
	UNetConnection* Connection = Actor->GetNetConnection();
	if (Connection)
	{
		const bool bIsReliable = EnumHasAnyFlags(Function->FunctionFlags, FUNC_NetReliable);

		// If we're saturated and it's not a reliable, drop it.
		if (!(bIsReliable || IsConnectionReady(Connection)))
		{
			return true;
		}

		// Route RPC calls to actual connection
		if (Connection->GetUChildConnection())
		{
			Connection = ((UChildConnection*)Connection)->Parent;
		}
	
		if (Connection->IsClosingOrClosed())
		{
			return true;
		}

		UActorChannel* Ch = Connection->FindActorChannelRef(Actor);
		if (Ch == nullptr)
		{
			if (Actor->IsPendingKillPending() || !NetDriver->IsLevelInitializedForActor(Actor, Connection))
			{
				// We can't open a channel for this actor here
				return true;
			}

			Ch = (UActorChannel *)Connection->CreateChannelByName( NAME_Actor, EChannelCreateFlags::OpenedLocally );
			if (!Ch)
			{
				if (!(Function->FunctionFlags & FUNC_NetReliable))
				{
					// Should we run out of channels then it should be ok to fail out for non-reliable functions
					return true;
				}
				else
				{
					// if reliable we want to give user feedback before exiting to allow further hosting
					Connection->Close(ENetCloseResult::ReplicationChannelCountMaxedOut);
					return true;
				}
			}

			Ch->SetChannelActor(Actor, ESetChannelActorFlags::None);
			
			if (UNetReplicationGraphConnection* ConnectionManager = Cast<UNetReplicationGraphConnection>(Connection->GetReplicationConnectionDriver()))
			{
				FConnectionReplicationActorInfo& ConnectionActorInfo = ConnectionManager->ActorInfoMap.FindOrAdd(Actor);
				FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(Actor);
				UpdateActorChannelCloseFrameNum(Actor, ConnectionActorInfo, GlobalInfo, ReplicationGraphFrame+1 /** Plus one to error on safe side. RepFrame num will be incremented in the next tick */, Connection );
			}
		}

		NetDriver->ProcessRemoteFunctionForChannel(Ch, ClassCache, FieldCache, TargetObj, Connection, Function, Parameters, OutParms, Stack, true);
	}
	else
	{
		UE_LOGF(LogNet, Warning, "UReplicationGraph::ProcessRemoteFunction: No owning connection for actor %ls. Function %ls will not be processed.", *Actor->GetName(), *Function->GetName());
	}
#endif // WITH_SERVER_CODE

	// return true because we don't want the net driver to do anything else
	return true;
}

void UReplicationGraph::PostTickDispatch()
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraph_PostTickDispatch);

	for (UNetConnection* NetConnection : ConnectionsNeedingsPostTickDispatchFlush)
	{
		if (NetConnection->GetDriver() != nullptr)
		{
			NetConnection->FlushNet();
		}
	}
	ConnectionsNeedingsPostTickDispatchFlush.Reset();
}

bool UReplicationGraph::IsConnectionReady(UNetConnection* Connection)
{
	if (CVar_RepGraph_DisableBandwithLimit)
	{
		return true;
	}

	return Connection->QueuedBits + Connection->SendBuffer.GetNumBits() <= 0;
}

void UReplicationGraph::SetActorDiscoveryBudget(int32 ActorDiscoveryBudgetInKBytesPerSec)
{
	// Disable the seperate actor discovery budget when 0
	if (ActorDiscoveryBudgetInKBytesPerSec <= 0)
	{
		ActorDiscoveryMaxBitsPerFrame = 0;
		UE_LOGF(LogReplicationGraph, Display, "SetActorDiscoveryBudget disabled the ActorDiscovery budget.");
		return;
	}

	if (NetDriver == nullptr)
	{
		UE_LOGF(LogReplicationGraph, Warning, "SetActorDiscoveryBudget ignored since NetDriver was not initialized.");
		return;
	}

	int32 MaxNetworkFPS = NetDriver->GetNetServerMaxTickRate();

	ActorDiscoveryMaxBitsPerFrame = (ActorDiscoveryBudgetInKBytesPerSec * 1000 * 8) / MaxNetworkFPS;
	UE_LOGF(LogReplicationGraph, Display, "SetActorDiscoveryBudget set to %d kBps (%d bits per network tick).", ActorDiscoveryBudgetInKBytesPerSec, ActorDiscoveryMaxBitsPerFrame);
}

void UReplicationGraph::SetAllCullDistanceSettingsForActor(const FActorRepListType& Actor, float CullDistanceSquared)
{
	FGlobalActorReplicationInfo& GlobalInfo = GlobalActorReplicationInfoMap.Get(Actor);
	GlobalInfo.Settings.SetCullDistanceSquared(CullDistanceSquared);

	for (UNetReplicationGraphConnection* RepGraphConnection : Connections)
	{
		if (FConnectionReplicationActorInfo* ConnectionActorInfo = RepGraphConnection->ActorInfoMap.Find(Actor))
		{
			ConnectionActorInfo->SetCullDistanceSquared(CullDistanceSquared);
		}
	}
}

void UReplicationGraph::NotifyConnectionSaturated(UNetReplicationGraphConnection& Connection)
{
	bWasConnectionSaturated = true;
	++GNumSaturatedConnections;
}

void UReplicationGraph::NotifyConnectionFastPathSaturated()
{
	bWasConnectionFastPathSaturated = true;
}

void UReplicationGraph::SetActorDestructionInfoToIgnoreDistanceCulling(AActor* DestroyedActor)
{
	if (!DestroyedActor)
	{
		return;
	}

	check(NetDriver);
	FNetworkGUID NetGUID = NetDriver->GetNetGuidCache()->GetNetGUID(DestroyedActor);
			
	if (NetGUID.IsDefault())
	{
		UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Warning, "SetActorDestructionInfoToIgnoreDistanceCulling ignored for %ls. No NetGUID assigned to the actor", *GetNameSafe(DestroyedActor));
		return;
	}

	// See if a destruction info exists for this actor
	if (const TUniquePtr<FActorDestructionInfo>* DestructionInfoPtr = NetDriver->DestroyedStartupOrDormantActors.Find(NetGUID))
	{
		(*DestructionInfoPtr)->bIgnoreDistanceCulling = true;
	}
}

void UReplicationGraph::PostServerReplicateStats(const FFrameReplicationStats& Stats)
{
#if CSV_PROFILER_STATS
	if (FCsvProfiler* Profiler = FCsvProfiler::Get())
	{
		if (Profiler->IsCapturing())
		{
			Profiler->RecordCustomStat("ReplicatedActors", CSV_CATEGORY_INDEX(ReplicationGraph), Stats.NumReplicatedActors, ECsvCustomStatOp::Set);
			Profiler->RecordCustomStat("CleanActors", CSV_CATEGORY_INDEX(ReplicationGraph), Stats.NumReplicatedCleanActors, ECsvCustomStatOp::Set);
			Profiler->RecordCustomStat("FastPathActors", CSV_CATEGORY_INDEX(ReplicationGraph), Stats.NumReplicatedFastPathActors, ECsvCustomStatOp::Set);
		}
	}
#endif
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------


UNetReplicationGraphConnection::UNetReplicationGraphConnection()
{

}

void UNetReplicationGraphConnection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UNetReplicationGraphConnection::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ActorInfoMap", ActorInfoMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("OnClientVisibleLevelNameAddMap", OnClientVisibleLevelNameAddMap.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingDestructionInfoList",
			PendingDestructInfoList.CountBytes(Ar);
			for (const FCachedDestructInfo& Info : PendingDestructInfoList)
			{
				Info.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("TrackedDestructionInfoPtrs", TrackedDestructionInfoPtrs.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingDormantDestructList", PendingDormantDestructList.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("TrackedDormantDestructionInfos", TrackedDormantDestructionInfos.CountBytes(Ar));
	}
}

void UNetReplicationGraphConnection::TearDown()
{
#if DO_ENABLE_REPGRAPH_DEBUG_ACTOR
	if (DebugActor.IsValid())
	{
		DebugActor->Destroy();
	}
	DebugActor = nullptr;
#endif

	Super::TearDown();

}
void UNetReplicationGraphConnection::NotifyActorChannelAdded(AActor* Actor, class UActorChannel* Channel)
{
#if WITH_SERVER_CODE
	UpdateActorConnectionCounter(Actor, Channel->Connection, ActorChannelCreateCounter);

	if (RepGraphConditionalActorBreakpoint(Actor, Channel->Connection))
	{
		UE_LOGF(LogReplicationGraph, Display, "::NotifyActorChannelAdded. %ls. Channel: %ls. Connection: %ls", *GetPathNameSafe(Actor), *Channel->Describe(), *Channel->Connection->Describe());
	}

	FConnectionReplicationActorInfo& ActorInfo = ActorInfoMap.FindOrAdd(Actor);

	// The ActorInfoMap may have a channel already.
	// This may happen in cases like dormancy where new Actor Channels can be created and then
	// closed multiple times for the same Actor, potentially before receiving CleanUp calls.
	if (ActorInfo.Channel && Channel != ActorInfo.Channel)
	{
		UE_LOGF(LogReplicationGraph, Log, "::NotifyActorChannelAdded. Fixing up stale channel reference Old: %ls New: %ls", *ActorInfo.Channel->Describe(), *Channel->Describe());
		ensureMsgf(ActorInfo.Channel->Closing, TEXT("Attempted to add an Actor Channel when a valid channel already exists for the actor. Actor=%s, OldChannel=%s, NewChannel=%s"),
			*GetPathNameSafe(Actor), *ActorInfo.Channel->Describe(), *Channel->Describe());

		ActorInfoMap.RemoveChannel(ActorInfo.Channel);
	}

	ActorInfo.Channel = Channel;
	ActorInfoMap.AddChannel(Actor, Channel);
#endif // WITH_SERVER_CODE
}

void UNetReplicationGraphConnection::NotifyActorChannelRemoved(AActor* Actor)
{
	// No need to do anything here. This is called when an actor channel is closed, but
	// we're still waiting for the close bunch to be acked. Until then, we can't safely replicate
	// the actor from this channel. See NotifyActorChannelCleanedUp.
}

void UNetReplicationGraphConnection::NotifyActorChannelCleanedUp(UActorChannel* Channel)
{
	if (Channel)
	{
		QUICK_SCOPE_CYCLE_COUNTER(UNetReplicationGraphConnection_NotifyActorChannelCleanedUp);

		FConnectionReplicationActorInfo* ActorInfo = ActorInfoMap.FindByChannel(Channel);
		if (ActorInfo)
		{
			// Note we don't remove the entry from ActorInfoMap.ActorMap here since the channel does not hold the Actor anymore.
			// The ActorMap entry will be removed by RemoveNetworkActor instead.
			
			if (Channel == ActorInfo->Channel)
			{
				// Only reset our state if we're still the associated channel.
				// We want to preserve the dormancy flag and the settings we pulled from the FGlobalActorReplicationInfo, but clear the frame counters, etc
				ActorInfo->ResetFrameCounters();
			}
			else
			{
				UE_LOGF(LogReplicationGraph, Log, "::NotifyActorChannelCleanedUp. CleanUp for stale channel reference Old: %ls New: %ls", *Channel->Describe(), *DescribeSafe(ActorInfo->Channel));
			}

			// Remove reference from channel map
			// ActorInfo should not be accessed after this since it could be destroyed.
			ActorInfoMap.RemoveChannel(Channel);
		}
	}
}

void UNetReplicationGraphConnection::InitForGraph(UReplicationGraph* Graph)
{
	// The per-connection data needs to know about the global data map so that it can pull defaults from it when we initialize a new actor
	TSharedPtr<FReplicationGraphGlobalData> Globals = Graph ? Graph->GetGraphGlobals() : nullptr;
	if (Globals.IsValid())
	{
		ActorInfoMap.SetGlobalMap(Globals->GlobalActorReplicationInfoMap);
	}
}

void UNetReplicationGraphConnection::InitForConnection(UNetConnection* InConnection)
{
	NetConnection = InConnection;
	InConnection->SetReplicationConnectionDriver(this);

#if DO_ENABLE_REPGRAPH_DEBUG_ACTOR
	UReplicationGraph* Graph = Cast<UReplicationGraph>(GetOuter());
	DebugActor = Graph->CreateDebugActor();
	if (DebugActor.IsValid())
	{
		DebugActor->ConnectionManager = this;
		DebugActor->ReplicationGraph = Graph;
	}
#endif

#if 0
	// This does not work because the control channel hasn't been opened yet. Could be moved further down the init path or in ServerReplicateActors.
	FString TestStr(TEXT("Replication Graph is Enabled!"));	
	FNetControlMessage<NMT_DebugText>::Send(InConnection,TestStr);
	InConnection->FlushNet();
#endif
}

void UNetReplicationGraphConnection::AddConnectionGraphNode(UReplicationGraphNode* Node)
{
	ConnectionGraphNodes.Add(Node);
}

void UNetReplicationGraphConnection::RemoveConnectionGraphNode(UReplicationGraphNode* Node)
{
	ConnectionGraphNodes.RemoveSingleSwap(Node);
	CleanupNodeCaches(Node);
}

bool UNetReplicationGraphConnection::PrepareForReplication()
{
	NetConnection->ViewTarget = NetConnection->PlayerController ? NetConnection->PlayerController->GetViewTarget() : ToRawPtr(NetConnection->OwningActor);
	
	UWorld* CurrentWorld = GetWorld();
	UPackage* CurrentWorldPackage = CurrentWorld ? CurrentWorld->GetPackage() : nullptr;
	const bool bConnectionHasCorrectWorld = CurrentWorldPackage ? NetConnection->GetClientWorldPackageName() == CurrentWorldPackage->GetFName() : true;

	// Set any children viewtargets
	for (int32 i = 0; i < NetConnection->Children.Num(); ++i)
	{
		UNetConnection* CurChild = NetConnection->Children[i];
		CurChild->ViewTarget = CurChild->PlayerController ? CurChild->PlayerController->GetViewTarget() : ToRawPtr(CurChild->OwningActor);
	}

	BuildVisibleLevels();

	// -------------------------------------------
	//	Handle dynamic dormant destruction TTL
	// -------------------------------------------
	if (!NodesVisibleCells.IsEmpty())
	{
		// First pass decreases TTL
		for (TTuple<TObjectKey<UReplicationGraphNode>, TArray<FVisibleCellInfo>>& NodeCellInfoPair : NodesVisibleCells)
		{
			for (FVisibleCellInfo& CellInfo : NodeCellInfoPair.Value)
			{
				CellInfo.Lifetime -= (CellInfo.Lifetime > 0);
			}
		}

		// Second pass removes dead nodes
		for (auto Iterator = NodesVisibleCells.CreateIterator(); Iterator; ++Iterator)
		{
			TArray<FVisibleCellInfo>& CellInfos = Iterator.Value();
			if (CellInfos.IsEmpty())
			{
				Iterator.RemoveCurrent();
			}
		}
	}

	return !NetConnection->IsClosingOrClosed() && (NetConnection->ViewTarget != nullptr) && bConnectionHasCorrectWorld;
}

void UNetReplicationGraphConnection::BuildVisibleLevels()
{
	CachedVisibleLevels.Reset();

	CachedVisibleLevels.Append(NetConnection->ClientVisibleLevelNames);
	for (const UNetConnection* Child : NetConnection->Children)
	{
		if (Child)
		{
			// For sets, we don't have to worry about uniqueness due to the nature of the data structure.
			CachedVisibleLevels.Append(Child->ClientVisibleLevelNames);
		}
	}
}

void UNetReplicationGraphConnection::NotifyAddDestructionInfo(FActorDestructionInfo* DestructInfo)
{
#if WITH_SERVER_CODE
	if (DestructInfo->StreamingLevelName != NAME_None)
	{
		if (NetConnection->ClientVisibleLevelNames.Contains(DestructInfo->StreamingLevelName) == false)
		{
			// This client does not have this streaming level loaded. We should get notified again via UNetConnection::UpdateLevelVisibility
			// (This should be enough. Legacy system would add the info and then do the level check in ::ServerReplicateActors, but this should be unnecessary)
			return;
		}
	}

	bool bWasAlreadyTracked = false;
	TrackedDestructionInfoPtrs.Add(DestructInfo, &bWasAlreadyTracked);
	if (bWasAlreadyTracked)
	{
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) 
	// Should not be happening but lets check in non shipping builds.
	int32 ExistingIdx = PendingDestructInfoList.IndexOfByKey(DestructInfo);	
	if (!ensureMsgf(ExistingIdx == INDEX_NONE, TEXT("::NotifyAddDestructionInfo already contains DestructInfo: 0x%" INT64_X_FMT " (%s)"), (int64)DestructInfo, *DestructInfo->PathName))
	{
		return;
	}
#endif

	PendingDestructInfoList.Emplace( FCachedDestructInfo(DestructInfo) );
	//UE_LOGF(LogReplicationGraph, Display, "::NotifyAddDestructionInfo. Connection: %ls. DestructInfo: %ls. NewTotal: %d", *NetConnection->Describe(), *DestructInfo->PathName, PendingDestructInfoList.Num());
#endif // WITH_SERVER_CODE
}

void UNetReplicationGraphConnection::NotifyAddDormantDestructionInfo(AActor* Actor)
{
#if WITH_SERVER_CODE
	if (Actor && NetConnection && NetConnection->Driver && NetConnection->Driver->GetNetGuidCache())
	{
		ULevel* Level = Actor->GetLevel();

		if (Level && !Level->IsPersistentLevel())
		{
			FName StreamingLevelName = Level->GetOutermost()->GetFName();

			if (NetConnection->ClientVisibleLevelNames.Contains(StreamingLevelName) == false)
			{
				UE_LOGF(LogReplicationGraph, Verbose, "NotifyAddDormantDestructionInfo skipping actor [%ls] because streaming level [%ls] is no longer visible.", *GetNameSafe(Actor), *StreamingLevelName.ToString());
				return;
			}
		}

		FNetworkGUID NetGUID = NetConnection->Driver->GetNetGuidCache()->GetNetGUID(Actor);
		if (NetGUID.IsValid() && !NetGUID.IsDefault())
		{
			bool bWasAlreadyTracked = false;
			TrackedDormantDestructionInfos.Add(NetGUID, &bWasAlreadyTracked);
			if (bWasAlreadyTracked)
			{
				return;
			}
	
			FCachedDormantDestructInfo& Info = PendingDormantDestructList.AddDefaulted_GetRef();

			Info.NetGUID = NetGUID;
			Info.Level = Level;
			Info.ObjOuter = Actor->GetOuter();
			Info.PathName = Actor->GetName();

			if (CVar_RepGraph_MarkDormantDestroyedActorsActive)
			{
				// MarkActive here to get this connection off the actor's DormantConnections list.
				// Since the actor will be destroyed, it shouldn't be considered dormant anymore.
				// If FlushNetDormancy is called later for this actor, this prevents the flush
				// from creating a replicator based on the actor's current state - which wouldn't
				// work, becuase the client won't have the current state.
				NetConnection->Driver->GetNetworkObjectList().MarkActive(Actor, NetConnection, NetConnection->Driver);
			}
		}
	}
#endif // WITH_SERVER_CODE
}

void UNetReplicationGraphConnection::NotifyRemoveDestructionInfo(FActorDestructionInfo* DestructInfo)
{
	const FCachedDestructInfo CachedDestructInfo(DestructInfo);

	bool bRemoved = PendingDestructInfoList.RemoveSingleSwap(DestructInfo, EAllowShrinking::No) > 0;

	// Check if the actor is in the out of range list
	if( !bRemoved )
	{
		OutOfRangeDestroyedActors.RemoveSingleSwap(DestructInfo, EAllowShrinking::No);
	}
	
	TrackedDestructionInfoPtrs.Remove(DestructInfo);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Check that its totally gone. Should not be happening!
	int32 DuplicateIdx = INDEX_NONE;
	while(true)
	{
		DuplicateIdx = PendingDestructInfoList.IndexOfByKey(DestructInfo);
		if (!ensureMsgf(DuplicateIdx == INDEX_NONE, TEXT("::NotifyRemoveDestructionInfo list STILL contains DestructInfo: 0x%llX (%s)"), (int64)DestructInfo, *DestructInfo->PathName))
		{
			PendingDestructInfoList.RemoveAtSwap(DuplicateIdx, EAllowShrinking::No);
			continue;
		}
		break;
	}
#endif
}

void UNetReplicationGraphConnection::NotifyResetDestructionInfo()
{
	TrackedDestructionInfoPtrs.Reset();
	PendingDestructInfoList.Reset();
	OutOfRangeDestroyedActors.Reset();
}

void UNetReplicationGraphConnection::NotifyResetAllNetworkActors()
{
	for (UReplicationGraphNode* Node : ConnectionGraphNodes)
	{
		Node->NotifyResetAllNetworkActors();
	}

	// Intentionally not resetting the actor info map, since it may contain references to open channels 
	// Open channels associated with actors that are not added back into the graph will be cleaned up normally using the close frame number
}

FActorRepListRefView& UNetReplicationGraphConnection::GetPrevDormantActorListForNode(const UReplicationGraphNode* GridNode)
{
	return PrevDormantActorListPerNode.FindOrAdd(GridNode);
}

void UNetReplicationGraphConnection::RemoveActorFromAllPrevDormantActorLists(AActor* InActor)
{
	for (TPair<TObjectKey<UReplicationGraphNode>, FActorRepListRefView>& PrevDormantActorListPerGridPair : PrevDormantActorListPerNode)
	{
		PrevDormantActorListPerGridPair.Value.RemoveFast(InActor);
	}
}

TArray<UNetReplicationGraphConnection::FVisibleCellInfo>& UNetReplicationGraphConnection::GetVisibleCellsForNode(const UReplicationGraphNode* GridNode)
{
	return NodesVisibleCells.FindOrAdd(GridNode);
}

void UNetReplicationGraphConnection::CleanupNodeCaches(const UReplicationGraphNode* Node)
{
	NodesVisibleCells.Remove(Node);
	PrevDormantActorListPerNode.Remove(Node);
}

void UNetReplicationGraphConnection::TrackReplicationForAnalytics(bool bFastPathSaturated)
{
	FastPathSaturationAnalytics.TrackReplication(bFastPathSaturated);
}

void UNetReplicationGraphConnection::ResetSaturationAnalytics()
{
	FastPathSaturationAnalytics = FRepGraphConnectionSaturationAnalytics();
}

void UNetReplicationGraphConnection::GetClientVisibleLevelNames(TSet<FName>& OutLevelNames) const
{
	if (NetConnection == nullptr)
	{
		return;
	}

	OutLevelNames.Append(NetConnection->ClientVisibleLevelNames);
	for (const UNetConnection* Child : NetConnection->Children)
	{
		if (Child != nullptr)
		{
			// For sets, we don't have to worry about uniqueness due to the nature of the data structure.
			OutLevelNames.Append(Child->ClientVisibleLevelNames);
		}
	}
}

void UNetReplicationGraphConnection::NotifyClientVisibleLevelNamesAdd(FName LevelName, UWorld* StreamingWorld) 
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(UNetReplicationGraphConnection_NotifyClientVisibleLevelNamesAdd);
	// Undormant every actor in this world for this connection.
	if (StreamingWorld && StreamingWorld->PersistentLevel)
	{
		auto& Actors = StreamingWorld->PersistentLevel->Actors;
		for (AActor* Actor : Actors)
		{
			if (Actor && (Actor->NetDormancy == DORM_DormantAll || (Actor->NetDormancy == DORM_Initial && Actor->IsNetStartupActor() == false)))
			{
				SetActorNotDormantOnConnection(Actor);
			}
		}
	}

	OnClientVisibleLevelNameAdd.Broadcast(LevelName, StreamingWorld);
	if (FOnClientVisibleLevelNamesAdd* MapDelegate = OnClientVisibleLevelNameAddMap.Find(LevelName))
	{
		MapDelegate->Broadcast(LevelName, StreamingWorld);
	}
}

int64 UNetReplicationGraphConnection::ReplicateDestructionInfos(const FRepGraphDestructionViewerInfoArray& DestructionViewersInfo, const FReplicationGraphDestructionSettings& DestructionSettings)
{
	int64 NumBits = 0;
#if WITH_SERVER_CODE

	for (int32 idx=PendingDestructInfoList.Num()-1; idx >=0; --idx)
	{
		FCachedDestructInfo& Info = PendingDestructInfoList[idx];
		FActorDestructionInfo* DestructInfo = Info.DestructionInfo;
		bool bSendDestructionInfo = false;
		bool bAddToOutOfRangeList = true;

		if (!DestructInfo->bIgnoreDistanceCulling)
		{
			// Only send destruction info if the viewers are close enough to the destroyed actor
			for (const FRepGraphDestructionViewerInfo& Viewer : DestructionViewersInfo)
			{
				const FVector::FReal DistSquared = FVector::DistSquared2D(Info.CachedPosition, Viewer.ViewerLocation);

				if (DistSquared < DestructionSettings.DestructInfoMaxDistanceSquared)
				{
					bSendDestructionInfo = true;
					break;
				}

				const FVector::FReal OutOfRangeDistSquared = FVector::DistSquared2D(Info.CachedPosition, Viewer.LastOutOfRangeLocationCheck);
				
				// Add the actor to the OutOfRangeList only if it is outside the range from the next check. If not, keep it in the destruction list to be evaluated next frame.
				if (OutOfRangeDistSquared <= DestructionSettings.MaxPendingListDistanceSquared)
				{
					bAddToOutOfRangeList = false;
				}
			}
		}

		if (bSendDestructionInfo || DestructInfo->bIgnoreDistanceCulling)
		{
			if (NetConnection && NetConnection->Driver)
			{
				NumBits += NetConnection->Driver->SendDestructionInfo(NetConnection, DestructInfo);

				PendingDestructInfoList.RemoveAtSwap(idx, EAllowShrinking::No);
				TrackedDestructionInfoPtrs.Remove(DestructInfo);
			}
		}
		else if (bAddToOutOfRangeList)
		{
			// Add the far actor to the out of range list so we don't evaluate it every frame
			OutOfRangeDestroyedActors.Emplace(MoveTemp(Info));
			PendingDestructInfoList.RemoveAtSwap(idx, EAllowShrinking::No);
		}
	}
#endif // #if WITH_SERVER_CODE
	return NumBits;
}

int64 UNetReplicationGraphConnection::ReplicateDormantDestructionInfos()
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ReplicateDormantDestructionInfos);

	int64 NumBits = 0;

#if WITH_SERVER_CODE

	if (NetConnection && NetConnection->Driver)
	{
		for (const FCachedDormantDestructInfo& Info : PendingDormantDestructList)
		{
			FActorDestructionInfo DestructInfo;
			DestructInfo.DestroyedPosition = FVector::ZeroVector;
			DestructInfo.NetGUID = Info.NetGUID;
			DestructInfo.Level = Info.Level;
			DestructInfo.ObjOuter = Info.ObjOuter;
			DestructInfo.PathName = Info.PathName;
			DestructInfo.StreamingLevelName = NAME_None;			// currently unused by SetChannelActorForDestroy
			DestructInfo.Reason = EChannelCloseReason::Relevancy;

			NumBits += NetConnection->Driver->SendDestructionInfo(NetConnection, &DestructInfo);
		}

		PendingDormantDestructList.Reset();
		TrackedDormantDestructionInfos.Reset();
	}
#endif // WITH_SERVER_CODE

	return NumBits;
}

void UNetReplicationGraphConnection::UpdateGatherLocationsForConnection(const FNetViewerArray& ConnectionViewers, const FReplicationGraphDestructionSettings& DestructionSettings)
{
	for (const FNetViewer& CurViewer : ConnectionViewers)
	{
		if (CurViewer.Connection != nullptr)
		{
			FLastLocationGatherInfo* LastInfoForViewer = LastGatherLocations.FindByKey<UNetConnection*>(CurViewer.Connection);
			if (LastInfoForViewer != nullptr)
			{
				OnUpdateViewerLocation(LastInfoForViewer, CurViewer, DestructionSettings);
			}
			else
			{
				// We need to add this viewer to the last gather locations
				LastGatherLocations.Emplace(CurViewer.Connection, CurViewer.ViewLocation);
			}
		}
	}

	// Clean up any dead entries in the last gather array
	LastGatherLocations.RemoveAll([&](FLastLocationGatherInfo& CurGatherInfo) {
		return CurGatherInfo.Connection == nullptr;
	});
}

void UNetReplicationGraphConnection::OnUpdateViewerLocation(FLastLocationGatherInfo* LocationInfo, const FNetViewer& Viewer, const FReplicationGraphDestructionSettings& DestructionSettings )
{
	const bool bIgnoreDistanceCheck = DestructionSettings.OutOfRangeDistanceCheckThresholdSquared == 0.0f;

	const FVector::FReal OutOfRangeDistanceSquared = FVector::DistSquared2D(Viewer.ViewLocation, LocationInfo->LastOutOfRangeLocationCheck);

	// Test all accumulated out of range actors only once the viewer has gone far enough from the last check
	if( bIgnoreDistanceCheck || (OutOfRangeDistanceSquared > DestructionSettings.OutOfRangeDistanceCheckThresholdSquared) )
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(OnUpdateViewerLocation_TestDestroyedActors);

		LocationInfo->LastOutOfRangeLocationCheck = Viewer.ViewLocation;

		for (int32 Index=OutOfRangeDestroyedActors.Num()-1; Index >=0; --Index)
		{
			FCachedDestructInfo& CachedInfo = OutOfRangeDestroyedActors[Index];

			const FVector::FReal ActorDistSquared = FVector::DistSquared2D(CachedInfo.CachedPosition, Viewer.ViewLocation);
				
			if (ActorDistSquared <= DestructionSettings.MaxPendingListDistanceSquared)
			{
				// Swap the info into the Pending List to get it replicated
				PendingDestructInfoList.Emplace(MoveTemp(CachedInfo));
				OutOfRangeDestroyedActors.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
	}

	LocationInfo->LastLocation = Viewer.ViewLocation;
}

void UNetReplicationGraphConnection::SetActorNotDormantOnConnection(AActor* InActor)
{
	if (FConnectionReplicationActorInfo* Info = ActorInfoMap.Find(InActor))
	{
		Info->bDormantOnConnection = false;
		Info->bGridSpatilization_AlreadyDormant = false;
		Info->bForceCullDistanceToZero = false;
		RemoveActorFromAllPrevDormantActorLists(InActor);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

UReplicationGraphNode::UReplicationGraphNode()
{

}

void UReplicationGraphNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraphNode::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("AllChildNodes", AllChildNodes.CountBytes(Ar));
	}
}

void UReplicationGraphNode::NotifyResetAllNetworkActors()
{
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		ChildNode->NotifyResetAllNetworkActors();
	}
}

bool UReplicationGraphNode::RemoveChildNode(UReplicationGraphNode* ChildNode, UReplicationGraphNode::NodeOrdering NodeOrder)
{
	ensure(ChildNode != nullptr);

	int32 Removed(0);
	
	if (NodeOrder == NodeOrdering::IgnoreOrdering)
	{
		Removed = AllChildNodes.RemoveSingleSwap(ChildNode, EAllowShrinking::No);
	}
	else
	{
		Removed = AllChildNodes.RemoveSingle(ChildNode);
	}

	if (Removed > 0)
	{
		ChildNode->TearDown();
	}

	return Removed > 0;
}

void UReplicationGraphNode::CleanChildNodes(UReplicationGraphNode::NodeOrdering NodeOrder)
{
	auto RemoveFunc = [](UReplicationGraphNode* GridChildNode)
	{
		return !IsValid(GridChildNode);
	};

	if (NodeOrder == NodeOrdering::IgnoreOrdering)
	{
		AllChildNodes.RemoveAllSwap(RemoveFunc, EAllowShrinking::No);
	}
	else
	{
		AllChildNodes.RemoveAll(RemoveFunc);
	}

}

void UReplicationGraphNode::TearDown()
{
	for (UReplicationGraphNode* Node : AllChildNodes)
	{
		Node->TearDown();
	}

	AllChildNodes.Reset();

	MarkAsGarbage();
}

void UReplicationGraphNode::DoCollectActorRepListStats(FActorRepListStatCollector& StatsCollector) const
{
	// Visit lists owned by this node
	OnCollectActorRepListStats(StatsCollector);

	// Flag the node as visited so we don't collect it twice
	StatsCollector.FlagNodeVisited(this);

	// Collect stats on all child nodes too
	for (const UReplicationGraphNode* Node : AllChildNodes)
	{
		Node->DoCollectActorRepListStats(StatsCollector);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------
void FStreamingLevelActorListCollection::AddActor(const FNewReplicatedActorInfo& ActorInfo)
{
	FStreamingLevelActors* Item = StreamingLevelLists.FindByKey(ActorInfo.StreamingLevelName);
	if (!Item)
	{
		Item = &StreamingLevelLists.Emplace_GetRef(ActorInfo.StreamingLevelName);
	}

	if (CVar_RepGraph_Verify)
	{
		ensureMsgf(Item->ReplicationActorList.Contains(ActorInfo.Actor) == false, TEXT("%s being added to a list twice! Streaming level: %s"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *ActorInfo.StreamingLevelName.ToString() );
	}

	Item->ReplicationActorList.Add(ActorInfo.Actor);
}

bool FStreamingLevelActorListCollection::RemoveActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound, UObject* Outer)
{
	bool bRemovedSomething = false;
	for (FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (StreamingList.StreamingLevelName == ActorInfo.StreamingLevelName)
		{
			bRemovedSomething = StreamingList.ReplicationActorList.RemoveSlow(ActorInfo.Actor);
			if (!bRemovedSomething && bWarnIfNotFound)
			{
				UE_LOGF(LogReplicationGraph, Warning, "Attempted to remove %ls from list %ls but it was not found. (StreamingLevelName == %ls)", *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathNameSafe(Outer), *ActorInfo.StreamingLevelName.ToString() );
			}

			if (CVar_RepGraph_Verify)
			{
				ensureMsgf(StreamingList.ReplicationActorList.Contains(ActorInfo.Actor) == false, TEXT("Actor %s is still in list after removal. Streaming Level: %s"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathNameSafe(Outer));
			}
			break;
		}
	}
	return bRemovedSomething;
}

bool FStreamingLevelActorListCollection::RemoveActorFast(const FNewReplicatedActorInfo& ActorInfo, UObject* Outer)
{
	bool bRemovedSomething = false;
	for (FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (StreamingList.StreamingLevelName == ActorInfo.StreamingLevelName)
		{
			bRemovedSomething = StreamingList.ReplicationActorList.RemoveFast(ActorInfo.Actor);
			break;
		}
	}
	return bRemovedSomething;
}

bool FStreamingLevelActorListCollection::RemoveActorFromLevelFast(AActor* Actor, FName LevelName)
{
	bool bRemovedSomething = false;
	for (FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (StreamingList.StreamingLevelName == LevelName)
		{
			bRemovedSomething = StreamingList.ReplicationActorList.RemoveFast(Actor);
			break;
		}
	}
	return bRemovedSomething;
}

void FStreamingLevelActorListCollection::Reset()
{
	for (FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		StreamingList.ReplicationActorList.Reset();
	}
}

void FStreamingLevelActorListCollection::Gather(const FConnectionGatherActorListParameters& Params) const
{
	for (const FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
		{
			Params.OutGatheredReplicationLists.AddReplicationActorList(StreamingList.ReplicationActorList);
		}
		else
		{
			UE_LOGF(LogReplicationGraph, Verbose, "Level Not Loaded %ls. (Client has %d levels loaded)", *StreamingList.StreamingLevelName.ToString(), Params.ClientVisibleLevelNamesRef.Num());
		}
	}
}

void FStreamingLevelActorListCollection::Gather(const UNetReplicationGraphConnection& ConnectionManager, FGatheredReplicationActorLists& OutGatheredList) const
{
	const TSet<FName>& ClientLevelNames = ConnectionManager.GetCachedClientVisibleLevelNames();

	for (const FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (ClientLevelNames.Contains(StreamingList.StreamingLevelName))
		{
			OutGatheredList.AddReplicationActorList(StreamingList.ReplicationActorList);
		}
	}
}

void FStreamingLevelActorListCollection::AppendAllLists(FGatheredReplicationActorLists& OutGatheredList) const
{
	for (const FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		OutGatheredList.AddReplicationActorList(StreamingList.ReplicationActorList);
	}
}

bool FStreamingLevelActorListCollection::Contains(const FNewReplicatedActorInfo& ActorInfo) const
{
	for (const FStreamingLevelActors& StreamingList : StreamingLevelLists)
	{
		if (StreamingList.StreamingLevelName == ActorInfo.StreamingLevelName)
		{
			return StreamingList.ReplicationActorList.Contains(ActorInfo.Actor);
		}
	}

	return false;
}

void FStreamingLevelActorListCollection::DeepCopyFrom(const FStreamingLevelActorListCollection& Source)
{
	StreamingLevelLists.Reset();
	for (const FStreamingLevelActors& StreamingLevel : Source.StreamingLevelLists)
	{
		if (StreamingLevel.ReplicationActorList.Num() > 0)
		{
			FStreamingLevelActors& NewStreamingLevel = StreamingLevelLists.Emplace_GetRef(StreamingLevel.StreamingLevelName);
			NewStreamingLevel.ReplicationActorList.CopyContentsFrom(StreamingLevel.ReplicationActorList);
			ensure(NewStreamingLevel.ReplicationActorList.Num() == StreamingLevel.ReplicationActorList.Num());
		}
	}
}

void FStreamingLevelActorListCollection::GetAll_Debug(TArray<FActorRepListType>& OutArray) const
{
	for (const FStreamingLevelActors& StreamingLevel : StreamingLevelLists)
	{
		StreamingLevel.ReplicationActorList.AppendToTArray(OutArray);
	}
}

void FStreamingLevelActorListCollection::Log(FReplicationGraphDebugInfo& DebugInfo) const
{
	for (const FStreamingLevelActors& StreamingLevelList : StreamingLevelLists)
	{
		LogActorRepList(DebugInfo, StreamingLevelList.StreamingLevelName.ToString(), StreamingLevelList.ReplicationActorList);
	}
}

void FStreamingLevelActorListCollection::TearDown()
{
	for (FStreamingLevelActors& StreamingLevelList : StreamingLevelLists)
	{
		StreamingLevelList.ReplicationActorList.TearDown();
	}

	StreamingLevelLists.Empty();
}

// --------------------------------------------------------------------------------------------------------------------------------------------

void UReplicationGraphNode_ActorList::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraphNode_ActorList::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReplicationActorList", ReplicationActorList.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("StreamingLevelCollection", StreamingLevelCollection.CountBytes(Ar));
	}
}

void UReplicationGraphNode_ActorList::NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorAdd>0, LogReplicationGraph, Display, "UReplicationGraphNode_ActorList::NotifyAddNetworkActor %ls on %ls.", *ActorInfo.Actor->GetFullName(), *GetPathName());

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		if (CVar_RepGraph_Verify)
		{
			ensureMsgf(ReplicationActorList.Contains(ActorInfo.Actor) == false, TEXT("%s being added to %s twice!"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathName());
		}

		ReplicationActorList.Add(ActorInfo.Actor);
	}
	else
	{
		StreamingLevelCollection.AddActor(ActorInfo);
	}
}

bool UReplicationGraphNode_ActorList::NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_ActorList::NotifyRemoveNetworkActor %ls on %ls.", *ActorInfo.Actor->GetFullName(), *GetPathName());

	bool bRemovedSomething = false;

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		bRemovedSomething = ReplicationActorList.RemoveSlow(ActorInfo.Actor);

		UE_CLOGF(!bRemovedSomething && bWarnIfNotFound, LogReplicationGraph, Warning, "Attempted to remove %ls from list %ls but it was not found. (StreamingLevelName == NAME_None)", *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetFullName());

		if (CVar_RepGraph_Verify)
		{
			ensureMsgf(ReplicationActorList.Contains(ActorInfo.Actor) == false, TEXT("Actor %s is still in %s after removal"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathName());
		}
	}
	else
	{
		bRemovedSomething = StreamingLevelCollection.RemoveActor(ActorInfo, bWarnIfNotFound, this);
	}

	return bRemovedSomething;
}

bool UReplicationGraphNode_ActorList::NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	const bool bRemovedSomething = NotifyRemoveNetworkActor(ActorInfo.OldActorInfo, bWarnIfNotFound);
	if (bRemovedSomething)
	{
		NotifyAddNetworkActor(ActorInfo.NewActorInfo);
	}

	return bRemovedSomething;
}

/** Removes the actor very quickly but breaks the list order */
bool UReplicationGraphNode_ActorList::RemoveNetworkActorFast(const FNewReplicatedActorInfo& ActorInfo)
{
	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		return ReplicationActorList.RemoveFast(ActorInfo.Actor);
	}
	else
	{
		return StreamingLevelCollection.RemoveActorFast(ActorInfo, this);
	}
}
	
void UReplicationGraphNode_ActorList::NotifyResetAllNetworkActors()
{
	ReplicationActorList.Reset();
	StreamingLevelCollection.Reset();
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		ChildNode->NotifyResetAllNetworkActors();
	}
}

void UReplicationGraphNode_ActorList::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
	Params.OutGatheredReplicationLists.AddReplicationActorList(ReplicationActorList);
	StreamingLevelCollection.Gather(Params);
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		ChildNode->GatherActorListsForConnection(Params);
	}
}

void UReplicationGraphNode_ActorList::DeepCopyActorListsFrom(const UReplicationGraphNode_ActorList* Source)
{
	if (Source->ReplicationActorList.Num() > 0)
	{
		ReplicationActorList.CopyContentsFrom(Source->ReplicationActorList);
	}

	StreamingLevelCollection.DeepCopyFrom(Source->StreamingLevelCollection);
}

void UReplicationGraphNode_ActorList::GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const
{
	ReplicationActorList.AppendToTArray(OutArray);
	StreamingLevelCollection.GetAll_Debug(OutArray);
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		ChildNode->GetAllActorsInNode_Debugging(OutArray);
	}
}

void UReplicationGraphNode_ActorList::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const FActorRepListType& Actor : ReplicationActorList)
	{
		VerifyActorReference(Actor);
	}

	for (const FStreamingLevelActorListCollection::FStreamingLevelActors& Item : StreamingLevelCollection.StreamingLevelLists)
	{
		for (const FActorRepListType& Actor : Item.ReplicationActorList)
		{
			VerifyActorReference(Actor);
		}
	}
}

void UReplicationGraphNode_ActorList::TearDown()
{
	Super::TearDown();

	ReplicationActorList.TearDown();
	StreamingLevelCollection.TearDown();
}

void UReplicationGraphNode_ActorList::OnCollectActorRepListStats(FActorRepListStatCollector& StatsCollector) const
{
	StatsCollector.VisitRepList(this, ReplicationActorList);
	StatsCollector.VisitStreamingLevelCollection(this, StreamingLevelCollection);

	Super::OnCollectActorRepListStats(StatsCollector);
}

void UReplicationGraphNode_ActorList::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);
	DebugInfo.PushIndent();	

	LogActorList(DebugInfo);

	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		DebugInfo.PushIndent();
		ChildNode->LogNode(DebugInfo, FString::Printf(TEXT("Child: %s"), *ChildNode->GetName()));
		DebugInfo.PopIndent();
	}
	DebugInfo.PopIndent();
}

void UReplicationGraphNode_ActorList::LogActorList(FReplicationGraphDebugInfo& DebugInfo) const
{
	LogActorRepList(DebugInfo, TEXT("World"), ReplicationActorList);
	StreamingLevelCollection.Log(DebugInfo);
}

// --------------------------------------------------------------------------------------------------------------------------------------------


UReplicationGraphNode_ActorListFrequencyBuckets::FSettings UReplicationGraphNode_ActorListFrequencyBuckets::DefaultSettings;

void UReplicationGraphNode_ActorListFrequencyBuckets::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraphNode_ActorListFrequencyBuckets::Serialize");
		if (Settings.IsValid())
		{
			GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Settings", Settings->CountBytes(Ar));
		}

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NonStreamingCollection",
			NonStreamingCollection.CountBytes(Ar);
			for (const FActorRepListRefView& RepList : NonStreamingCollection)
			{
				RepList.CountBytes(Ar);
			}
		);
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("StreamingLevelCollection", StreamingLevelCollection.CountBytes(Ar));
	}
}

void UReplicationGraphNode_ActorListFrequencyBuckets::NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorAdd>0, LogReplicationGraph, Display, "UReplicationGraphNode_ActorListFrequencyBuckets::NotifyAddNetworkActor %ls on %ls.", *ActorInfo.Actor->GetFullName(), *GetPathName());

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		// Add to smallest bucket
		FActorRepListRefView* BestList = nullptr;
		int32 LeastNum = INT_MAX;
		for (FActorRepListRefView& List : NonStreamingCollection)
		{
			if (List.Num() < LeastNum)
			{
				BestList = &List;
				LeastNum = List.Num();
			}

			if (CVar_RepGraph_Verify)
			{
				ensureMsgf(List.Contains(ActorInfo.Actor) == false, TEXT("%s being added to %s twice!"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathName());
			}
		}

		repCheck(BestList != nullptr);
		BestList->Add(ActorInfo.Actor);
		TotalNumNonStreamingActors++;
		CheckRebalance();
	}
	else
	{
		StreamingLevelCollection.AddActor(ActorInfo);
	}
}

bool UReplicationGraphNode_ActorListFrequencyBuckets::NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_ActorListFrequencyBuckets::NotifyRemoveNetworkActor %ls on %ls.", *ActorInfo.Actor->GetFullName(), *GetPathName());

	bool bRemovedSomething = false;
	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		bool bFound = false;
		for (FActorRepListRefView& List : NonStreamingCollection)
		{
			if (List.RemoveSlow(ActorInfo.Actor))
			{
				bRemovedSomething = true;
				TotalNumNonStreamingActors--;
				CheckRebalance();

				if (!CVar_RepGraph_Verify)
				{
					// Eary out if we dont have to verify
					return bRemovedSomething;
				}

				if (bFound)
				{
					// We already removed this actor so this is a dupe!
					repCheck(CVar_RepGraph_Verify);
					ensureMsgf(false, TEXT("Actor %s is still in %s after removal"), *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetPathName());
				}

				bFound = true;
			}
		}

		if (!bFound && bWarnIfNotFound)
		{
			UE_LOGF(LogReplicationGraph, Warning, "Attempted to remove %ls from list %ls but it was not found. (StreamingLevelName == NAME_None)", *GetActorRepListTypeDebugString(ActorInfo.Actor), *GetFullName());
		}
	}
	else
	{
		bRemovedSomething = StreamingLevelCollection.RemoveActor(ActorInfo, bWarnIfNotFound, this);
	}

	return bRemovedSomething;
}
	
bool UReplicationGraphNode_ActorListFrequencyBuckets::NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	const bool bRemovedSomething = NotifyRemoveNetworkActor(ActorInfo.OldActorInfo, bWarnIfNotFound);
	if (bRemovedSomething)
	{
		NotifyAddNetworkActor(ActorInfo.NewActorInfo);
	}

	return bRemovedSomething;
}

void UReplicationGraphNode_ActorListFrequencyBuckets::NotifyResetAllNetworkActors()
{
	for (FActorRepListRefView& List : NonStreamingCollection)
	{
		List.Reset();
	}
	StreamingLevelCollection.Reset();
	TotalNumNonStreamingActors = 0;
}

void UReplicationGraphNode_ActorListFrequencyBuckets::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
#if WITH_SERVER_CODE
	const FSettings& MySettings = GetSettings();
	const uint32 ReplicationFrameNum = Params.ReplicationFrameNum;
	if (MySettings.EnableFastPath)
	{
		// Return one list as Default and the rest as FastShared
		const int32 DefaultReplicationIdx = (ReplicationFrameNum % NonStreamingCollection.Num());
		for (int32 idx = 0; idx < NonStreamingCollection.Num(); ++idx)
		{
			if (DefaultReplicationIdx == idx)
			{
				// Default Rep Path
				Params.OutGatheredReplicationLists.AddReplicationActorList(NonStreamingCollection[idx], EActorRepListTypeFlags::Default);
			}
			else
			{
				// Only do FastShared if modulo passes
				if (ReplicationFrameNum % MySettings.FastPathFrameModulo == 0)
				{
					Params.OutGatheredReplicationLists.AddReplicationActorList(NonStreamingCollection[idx], EActorRepListTypeFlags::FastShared);
				}
			}
		}
	}
	else
	{
		// Default path only: don't return lists in "off" frames.
		const int32 idx = Params.ReplicationFrameNum % NonStreamingCollection.Num();
		Params.OutGatheredReplicationLists.AddReplicationActorList(NonStreamingCollection[idx]);
	}

	StreamingLevelCollection.Gather(Params);
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_ActorListFrequencyBuckets::SetNonStreamingCollectionSize(const int32 NewSize)
{
	// Save everything off
	static TArray<FActorRepListType> FullList;
	FullList.Reset();

	for (FActorRepListRefView& List : NonStreamingCollection)
	{
		List.AppendToTArray(FullList);
	}

	// Reset
	NonStreamingCollection.SetNum(NewSize);
	for (FActorRepListRefView& List : NonStreamingCollection)
	{
		List.Reset(GetSettings().ListSize);
	}

	// Readd/Rebalance
	for (int32 idx=0; idx < FullList.Num(); ++idx)
	{
		NonStreamingCollection[idx % NewSize].Add( FullList[idx] );
	}
}

void UReplicationGraphNode_ActorListFrequencyBuckets::CheckRebalance()
{
	const int32 CurrentNumBuckets = NonStreamingCollection.Num();
	int32 DesiredNumBuckets = CurrentNumBuckets;

	for (const FSettings::FBucketThresholds& Threshold : GetSettings().BucketThresholds)
	{
		if (TotalNumNonStreamingActors <= Threshold.MaxActors)
		{
			DesiredNumBuckets = Threshold.NumBuckets;
			break;
		}
	}

	if (DesiredNumBuckets != CurrentNumBuckets)
	{
		//UE_LOGF(LogReplicationGraph, Display, "Rebalancing %ls for %d buckets (%d total actors)", *GetPathName(), DesiredNumBuckets, TotalNumNonStreamingActors);
		SetNonStreamingCollectionSize(DesiredNumBuckets);
	}
	
}

void UReplicationGraphNode_ActorListFrequencyBuckets::GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const
{
	for (const FActorRepListRefView& List : NonStreamingCollection)
	{
		List.AppendToTArray(OutArray);
	}
	StreamingLevelCollection.GetAll_Debug(OutArray);
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		ChildNode->GetAllActorsInNode_Debugging(OutArray);
	}
}

void UReplicationGraphNode_ActorListFrequencyBuckets::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const FActorRepListRefView& List : NonStreamingCollection)
	{
		for (const FActorRepListType& Actor : List)
		{
			VerifyActorReference(Actor);
		}
	}

	for (const auto& List : StreamingLevelCollection.StreamingLevelLists)
	{
		for (const auto& Actor : List.ReplicationActorList)
		{
			VerifyActorReference(Actor);
		}
	}
}

void UReplicationGraphNode_ActorListFrequencyBuckets::OnCollectActorRepListStats(FActorRepListStatCollector& StatsCollector) const
{
	for (const FActorRepListRefView& List : NonStreamingCollection)
	{
		StatsCollector.VisitRepList(this, List);
	}
	StatsCollector.VisitStreamingLevelCollection(this, StreamingLevelCollection);
	
	Super::OnCollectActorRepListStats(StatsCollector);
}

void UReplicationGraphNode_ActorListFrequencyBuckets::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);
	DebugInfo.PushIndent();
	int32 i=0;

	for (const FActorRepListRefView& List : NonStreamingCollection)
	{
		LogActorRepList(DebugInfo, FString::Printf(TEXT("World Bucket %d"), ++i), List);
	}
	StreamingLevelCollection.Log(DebugInfo);
	DebugInfo.PopIndent();
}

void UReplicationGraphNode_ActorListFrequencyBuckets::TearDown()
{
	Super::TearDown();

	for (FActorRepListRefView& List : NonStreamingCollection)
	{
		List.TearDown();
	}
	StreamingLevelCollection.TearDown();
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// Dynamic Spatial Frequency
// --------------------------------------------------------------------------------------------------------------------------------------------

/*
 *	Notes on Default Zone Values
 *		-Below values assume 30hz tick rate (the default UNetDriver::NetServerMaxTickRate value).
 *		-If you have a different tick rate, you should reinitialize this data structure yourself. See ReInitDynamicSpatializationSettingsCmd as an example of how to do this from a game project.
 *		-(Alternatively, you can make your own subclass of UReplicationGraphNode_DynamicSpatialFrequency or set UReplicationGraphNode_DynamicSpatialFrequency::Settings*.
 *		
 *	Overview of algorithm:
 *		1. Determine which zone you are in based on DOT product
 *		2. Calculate % of distance/NetCullDistance
 *		3. Map+clamp calculated % to MinPCT/MaxPCT.
 *		4. Take calculated % (between 0-1) and map to MinDistHz - MaxDistHz.
 *
 */

namespace RepGraphDynamicSpatialFrequency
{
	constexpr float AssumedTickRate = 30.f; // UNetDriver::NetServerMaxTickRate
	constexpr float TargetKBytesSec = 10.f; // 10K/sec
	constexpr int64 BitsPerFrame = (int64)((TargetKBytesSec * 1024.f * 8.f) / AssumedTickRate);
};

static TArray<UReplicationGraphNode_DynamicSpatialFrequency::FSpatializationZone>& DefaultSpatializationZones()
{
	static TArray<UReplicationGraphNode_DynamicSpatialFrequency::FSpatializationZone> Zones;
	Zones.Reset();

//                                          [Default]                     [FastShared]
//	     			 DOT    MinPCT MaxPCT  MinDistHz  MaxDistHz          MinDistHz  MaxDistHz
	Zones.Emplace(   0.00f, 0.05f, 0.10f,  1.f,       1.f,                0.f,      0.f,           RepGraphDynamicSpatialFrequency::AssumedTickRate);	// Behind viewer
	Zones.Emplace(   0.71f, 0.05f, 0.10f,  1.f,       1.f,                0.f,      0.f,           RepGraphDynamicSpatialFrequency::AssumedTickRate);	// In front but not quite in FOV
	Zones.Emplace(   1.00f, 0.10f, 0.50f,  5.f,       1.f,               20.f,      10.f,          RepGraphDynamicSpatialFrequency::AssumedTickRate);	// Directly in viewer's FOV

	return Zones;
}

static TArray<UReplicationGraphNode_DynamicSpatialFrequency::FSpatializationZone>& DefaultSpatializationZones_NoFastShared()
{
	static TArray<UReplicationGraphNode_DynamicSpatialFrequency::FSpatializationZone> Zones;
	Zones.Reset();

//                                          [Default]                   [FastShared (Disabled]
//	     			 DOT    MinPCT MaxPCT  MinDistHz  MaxDistHz          MinDistHz  MaxDistHz
	Zones.Emplace(   0.00f, 0.05f, 0.10f,   5.f,      1.f,               0.f,        0.f,          RepGraphDynamicSpatialFrequency::AssumedTickRate);	// Behind viewer
	Zones.Emplace(   0.71f, 0.05f, 0.10f,  10.f,      5.f,               0.f,        0.f,          RepGraphDynamicSpatialFrequency::AssumedTickRate);	// In front but not quite in FOV
	Zones.Emplace(   1.00f, 0.10f, 0.50f,  20.f,      5.f,               0.f,        0.f,          RepGraphDynamicSpatialFrequency::AssumedTickRate);	// Directly in viewer's FOV

	return Zones;
}

UReplicationGraphNode_DynamicSpatialFrequency::FSettings UReplicationGraphNode_DynamicSpatialFrequency::DefaultSettings(DefaultSpatializationZones(), DefaultSpatializationZones_NoFastShared(), RepGraphDynamicSpatialFrequency::BitsPerFrame );

FAutoConsoleCommand ReInitDynamicSpatializationSettingsCmd(TEXT("Net.RepGraph.DyanmicSpatialization.Reinit"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]()
{
	new (&UReplicationGraphNode_DynamicSpatialFrequency::DefaultSettings) UReplicationGraphNode_DynamicSpatialFrequency::FSettings(DefaultSpatializationZones(), DefaultSpatializationZones_NoFastShared(), RepGraphDynamicSpatialFrequency::BitsPerFrame );
}));

UReplicationGraphNode_DynamicSpatialFrequency::UReplicationGraphNode_DynamicSpatialFrequency()
{
	CSVStatName = "DynamicSpatialFrequencyGatherPrioritize";
}

int32 CVar_RepGraph_DynamicSpatialFrequency_UncapBandwidth = 0;
static FAutoConsoleVariableRef CVarRepGraphDynamicSpatialFrequencyUncapBandwidth(TEXT("Net.RepGraph.DynamicSpatialFrequency.UncapBandwidth"), CVar_RepGraph_DynamicSpatialFrequency_UncapBandwidth,
	TEXT("Uncaps bandwidth on UReplicationGraphNode_DynamicSpatialFrequency nodes."), ECVF_Default);

REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.DynamicSpatialFrequency.OpportunisticLoadBalance", CVar_RepGraph_DynamicSpatialFrequency_OpportunisticLoadBalance, 1, "Defers replication 1 frame in cases where many actors replicate on this frame but few on next frame.");

FORCEINLINE bool ReplicatesEveryFrame(const FConnectionReplicationActorInfo& ConnectionInfo, const bool CheckFastPath)
{
	return !(ConnectionInfo.ReplicationPeriodFrame > 1 && (!CheckFastPath || ConnectionInfo.FastPath_ReplicationPeriodFrame > 1));
}

void UReplicationGraphNode_DynamicSpatialFrequency::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
#if WITH_SERVER_CODE
	repCheck(GraphGlobals.IsValid());

	UReplicationGraph* RepGraph = GraphGlobals->ReplicationGraph;
	repCheck(RepGraph);
	repCheck(GraphGlobals->GlobalActorReplicationInfoMap);

	FGlobalActorReplicationInfoMap& GlobalMap = *GraphGlobals->GlobalActorReplicationInfoMap;
	UNetConnection* NetConnection = Params.ConnectionManager.NetConnection;
	FPerConnectionActorInfoMap& ConnectionActorInfoMap = Params.ConnectionManager.ActorInfoMap;
	const int32 FrameNum = Params.ReplicationFrameNum;
	int32 TotalNumActorsExpectedNextFrame = 0;
	int32& QueuedBits = NetConnection->QueuedBits;

	FSettings& MySettings = GetSettings();

	const int32 MaxNearestActors = MySettings.MaxNearestActors;

	// --------------------------------------------------------
	SortedReplicationList.Reset();
	NumExpectedReplicationsThisFrame = 0;
	NumExpectedReplicationsNextFrame = 0;

	bool DoFullGather = true;

	{
#if CSV_PROFILER_STATS
		FScopedCsvStatExclusive ScopedStat(CSVStatName);
#endif	

		// --------------------------------------------------------------------------------------------------------
		//	Two passes: filter list down to MaxNearestActors actors based on distance. Then calc freq and resort
		// --------------------------------------------------------------------------------------------------------
		if (MaxNearestActors >= 0 && !NetConnection->IsReplay())
		{
			int32 PossibleNumActors = ReplicationActorList.Num();;

			// Are we even over the limit?
			for (const FStreamingLevelActorListCollection::FStreamingLevelActors& StreamingList : StreamingLevelCollection.StreamingLevelLists)
			{
				if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
				{
					PossibleNumActors += StreamingList.ReplicationActorList.Num();
				}
			}

			if (PossibleNumActors > MaxNearestActors)
			{
				// We need to do an initial filtering pass over these actors based purely on distance (not time since last replicated, etc).
				// We will only replicate MaxNearestActors actors.
				QUICK_SCOPE_CYCLE_COUNTER(REPGRAPH_DynamicSpatialFrequency_Gather_WithCap);

				DoFullGather = false; // Don't do the full gather below. Just looking at SortedReplicationList is not enough because its possible no actors are due to replicate this frame.

				// Go through all lists, calc distance and cache FGlobalActorInfo*
				GatherActors_DistanceOnly(ReplicationActorList, GlobalMap, ConnectionActorInfoMap, Params);

				for (const FStreamingLevelActorListCollection::FStreamingLevelActors& StreamingList : StreamingLevelCollection.StreamingLevelLists)
				{
					if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
					{
						GatherActors_DistanceOnly(StreamingList.ReplicationActorList, GlobalMap, ConnectionActorInfoMap, Params);
					}
				}

				ensure(PossibleNumActors == SortedReplicationList.Num());

				// Sort list by distance, remove Num - MaxNearestActors from end
				SortedReplicationList.Sort();
				SortedReplicationList.SetNum(MaxNearestActors, EAllowShrinking::No);

				// Do rest of normal spatial calculations and resort
				for (int32 idx = SortedReplicationList.Num()-1; idx >= 0; --idx)
				{
					FDynamicSpatialFrequency_SortedItem& Item = SortedReplicationList[idx];
					AActor* Actor = Item.Actor;
					FGlobalActorReplicationInfo& GlobalInfo = *Item.GlobalInfo;
				
					CalcFrequencyForActor(Actor, RepGraph, Params.ConnectionManager, GlobalInfo, ConnectionActorInfoMap, MySettings, Params.Viewers, FrameNum, idx);
				}

				SortedReplicationList.Sort();
			}
		}

		// --------------------------------------------------------------------------------------------------------
		//	Single pass: RepList -> Sorted frequency list. No cap on max number of actors to replicate
		// --------------------------------------------------------------------------------------------------------
		if (DoFullGather)
		{
			// No cap on numbers of actors, just pull them directly
			QUICK_SCOPE_CYCLE_COUNTER(REPGRAPH_DynamicSpatialFrequency_Gather);	

			GatherActors(ReplicationActorList, GlobalMap, ConnectionActorInfoMap, Params, NetConnection);

			for (const FStreamingLevelActorListCollection::FStreamingLevelActors& StreamingList : StreamingLevelCollection.StreamingLevelLists)
			{
				if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
				{
					GatherActors(StreamingList.ReplicationActorList, GlobalMap, ConnectionActorInfoMap, Params, NetConnection);
				}
			}

			SortedReplicationList.Sort();
		}
	}

	// --------------------------------------------------------

	{
		QUICK_SCOPE_CYCLE_COUNTER(REPGRAPH_DynamicSpatialFrequency_Replicate);
		
		const int64 MaxBits = MySettings.MaxBitsPerFrame;
		int64 BitsWritten = 0;

		// This is how many "not every frame" actors we should replicate this frame. When assigning dynamic frequencies we also track who is due to rep this frame and next frame.
		// If this frame has more than the next frame expects, we will deffer half of those reps this frame. This will naturally tend to spread things out. It is not perfect, but low cost.
		// Note that when an actor is starved (missed a replication frame) they will not be counted for any of this.
		int32 OpportunisticLoadBalanceQuota = (NumExpectedReplicationsThisFrame - NumExpectedReplicationsNextFrame) >> 1;

		for (const FDynamicSpatialFrequency_SortedItem& Item : SortedReplicationList)
		{
			AActor* Actor = Item.Actor;
			FGlobalActorReplicationInfo& GlobalInfo = *Item.GlobalInfo;
			FConnectionReplicationActorInfo& ConnectionInfo = *Item.ConnectionInfo;

			if (!Actor || IsActorValidForReplication(Actor) == false)
			{
				continue;
			}


			if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
			{
				UE_LOGF(LogReplicationGraph, Display, "UReplicationGraphNode_DynamicSpatialFrequency_Connection Replication: %ls", *Actor->GetName());
			}

			if (UNLIKELY(ConnectionInfo.bTearOff))
			{
				continue;
			}

			if (CVar_RepGraph_DynamicSpatialFrequency_OpportunisticLoadBalance && OpportunisticLoadBalanceQuota > 0 && Item.FramesTillReplicate == 0 && !ReplicatesEveryFrame(ConnectionInfo, Item.EnableFastPath))
			{
				//UE_LOGF(LogReplicationGraph, Display, "[%d] Opportunistic Skip. %ls. OpportunisticLoadBalanceQuota: %d (%d, %d)", FrameNum, *Actor->GetName(), OpportunisticLoadBalanceQuota, NumExpectedReplicationsThisFrame, NumExpectedReplicationsNextFrame);
				OpportunisticLoadBalanceQuota--;
				continue;
			}

			// ------------------------------------------------------
			//	Default Replication
			// ------------------------------------------------------

			if (ReadyForNextReplication(ConnectionInfo, GlobalInfo, FrameNum))
			{
				BitsWritten += RepGraph->ReplicateSingleActor(Actor, ConnectionInfo, GlobalInfo, ConnectionActorInfoMap, Params.ConnectionManager, FrameNum);
				ConnectionInfo.FastPath_LastRepFrameNum = FrameNum; // Manually update this here, so that we don't fast rep next frame. When they line up, use default replication.
			}

			// ------------------------------------------------------
			//	Fast Path
			// ------------------------------------------------------
			
			else if (Item.EnableFastPath && ReadyForNextReplication_FastPath(ConnectionInfo, GlobalInfo, FrameNum))
			{
				const int32 FastSharedBits = IntCastChecked<int32>(RepGraph->ReplicateSingleActor_FastShared(Actor, ConnectionInfo, GlobalInfo, Params.ConnectionManager, FrameNum));
				QueuedBits -= FastSharedBits; // We are doing our own bandwidth limiting here, so offset the netconnection's tracking.
				BitsWritten += FastSharedBits;
			}

			// Bandwidth Cap
			if (BitsWritten > MaxBits && CVar_RepGraph_DynamicSpatialFrequency_UncapBandwidth == 0)
			{
				Params.ConnectionManager.NotifyDSFNodeSaturated(this);
				break;
			}
		}

		if (CVar_RepGraph_DynamicSpatialFrequency_UncapBandwidth > 0)
		{
			UE_LOGF(LogReplicationGraph, Warning, "Uncapped bandwidth usage of UReplicationGraphNode_DynamicSpatialFrequency = %lld bits -> %lld bytes -> %.2f KBytes/sec", BitsWritten, (BitsWritten+7) >> 3,  ((float)((BitsWritten+7)>>3)/1024.f) * ((float)GraphGlobals->ReplicationGraph->NetDriver->GetNetServerMaxTickRate()));
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_DynamicSpatialFrequency::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const FDynamicSpatialFrequency_SortedItem& Item : SortedReplicationList)
	{
		VerifyActorReference(Item.Actor);
	}
}

REPGRAPH_DEVCVAR_SHIPCONST(int32, "Net.RepGraph.DynamicSpatialFrequency.Draw", CVar_RepGraph_DynamicSpatialFrequency_Draw, 0, "");

int32 CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq = 0;
static FAutoConsoleVariableRef CVarRepGraphDynamicSpatialFrequencyForceMaxFreq(TEXT("Net.RepGraph.DynamicSpatialFrequency.ForceMaxFreq"), CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq, TEXT("Forces DSF to set max frame replication periods on all actors (1 frame rep periods). 1 = default replication. 2 = fast path. 3 = Both (effectively, default)"), ECVF_Default);

FORCEINLINE uint32 CalcDynamicReplicationPeriod(const float FinalPCT, const uint32 MinRepPeriod, const uint32 MaxRepPeriod, uint16& OutReplicationPeriodFrame, uint32& OutNextReplicationFrame, const uint32 LastRepFrameNum, const uint32 FrameNum, bool ForFastPath)
{
	const float PeriodRange = (float)(MaxRepPeriod - MinRepPeriod);
	const uint32 ExtraPeriod = (uint32)FMath::CeilToInt(PeriodRange * FinalPCT);
				
	const uint32 FinalPeriod = MinRepPeriod + ExtraPeriod;
	OutReplicationPeriodFrame = (uint16)FMath::Clamp<uint32>(FinalPeriod, 1, MAX_uint16);

	const uint32 NextRepFrameNum = LastRepFrameNum + OutReplicationPeriodFrame;
	OutNextReplicationFrame = NextRepFrameNum;


#if !(UE_BUILD_SHIPPING)
	ensureMsgf(OutReplicationPeriodFrame == FinalPeriod, TEXT("Overflow error when FinalPeriod(%u) was assigned to OutReplicationPeriodFrame(%u). RepPeriod values are probably too big"), FinalPeriod, OutReplicationPeriodFrame);

	if (CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq > 0)
	{
		if ((CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq == 1 && ForFastPath == 0) ||
			(CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq == 2 && ForFastPath == 1) ||
			CVar_RepGraph_DynamicSpatialFrequency_ForceMaxFreq == 3
			)
		{
			OutReplicationPeriodFrame = 1;
			OutNextReplicationFrame = FrameNum;
		}
	}
#endif

	return ExtraPeriod;
}

static TArray<FColor> DynamicSpatialFrequencyDebugColorArray = { FColor::Red, FColor::Green, FColor::Blue, FColor::Cyan, FColor::Orange, FColor::Purple };

void UReplicationGraphNode_DynamicSpatialFrequency::CalcFrequencyForActor(AActor* Actor, UReplicationGraph* RepGraph, UNetConnection* NetConnection, FGlobalActorReplicationInfo& GlobalInfo, FPerConnectionActorInfoMap& ConnectionMap, FSettings& MySettings, const FNetViewerArray& Viewers, const uint32 FrameNum, int32 ExistingItemIndex)
{
#if WITH_SERVER_CODE
	UNetReplicationGraphConnection* RepGraphConnection = RepGraph->FindConnectionManager(NetConnection);
	CalcFrequencyForActor(Actor, RepGraph, *RepGraphConnection, GlobalInfo, ConnectionMap, MySettings, Viewers, FrameNum, ExistingItemIndex);
#endif
}

void UReplicationGraphNode_DynamicSpatialFrequency::CalcFrequencyForActor(AActor* Actor, UReplicationGraph* RepGraph, const UNetReplicationGraphConnection& RepGraphConnection, FGlobalActorReplicationInfo& GlobalInfo, FPerConnectionActorInfoMap& ConnectionMap, FSettings& MySettings, const FNetViewerArray& Viewers, const uint32 FrameNum, int32 ExistingItemIndex)
{
#if WITH_SERVER_CODE
	FConnectionReplicationActorInfo& ConnectionInfo = ConnectionMap.FindOrAdd(Actor);

	UNetConnection* NetConnection = RepGraphConnection.NetConnection;

	// If we need to filter out the actor and it is already in the SortedReplicationList, we need to remove it (instead of just skipping/returning).
	auto RemoveExistingItem = [&ExistingItemIndex, this]()
	{
		if (ExistingItemIndex != INDEX_NONE)
		{
			SortedReplicationList.RemoveAtSwap(ExistingItemIndex, EAllowShrinking::No);
		}
	};

	// When adding we either create a new item or reconstruct an item at the existing index
	auto AddOrUpdateItem = [&ExistingItemIndex, &RepGraph, &FrameNum, &NetConnection, this](AActor* InActor, int32 InFramesTillReplicate, bool InEnableFastPath, FGlobalActorReplicationInfo* InGlobal, FConnectionReplicationActorInfo* InConnection)
	{
		// Update actor close frame number here in case the actor gets skipped in the replication loop
		RepGraph->UpdateActorChannelCloseFrameNum(InActor, *InConnection, *InGlobal, FrameNum, NetConnection);

		FDynamicSpatialFrequency_SortedItem* Item = ExistingItemIndex == INDEX_NONE ? &SortedReplicationList[SortedReplicationList.AddUninitialized()] : &SortedReplicationList[ExistingItemIndex];
		new (Item) FDynamicSpatialFrequency_SortedItem(InActor, InFramesTillReplicate, InEnableFastPath, InGlobal, InConnection);
	};

	// ------------------------------------------------------------------------------------------

	if (RepGraphConditionalActorBreakpoint(Actor, NetConnection))
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraphNode_DynamicSpatialFrequency::CalcFrequencyForActor: %ls", *Actor->GetName());
	}

	if (ConnectionInfo.bDormantOnConnection)
	{
		RemoveExistingItem();
		return;
	}

	FVector::FReal SmallestDistanceToActorSq = std::numeric_limits<FVector::FReal>::max();
	const FNetViewer* LowestDistanceViewer = nullptr;

	// Find the closest viewer to this item or the first viewer if there are no viewers closer.
	for (const FNetViewer& CurViewer : Viewers)
	{
		FVector::FReal CurDistance = (GlobalInfo.WorldLocation - CurViewer.ViewLocation).SizeSquared();
		if (LowestDistanceViewer == nullptr || CurDistance < SmallestDistanceToActorSq)
		{
			LowestDistanceViewer = &CurViewer;
			SmallestDistanceToActorSq = CurDistance;
		}
	}

	check(LowestDistanceViewer != nullptr);
	UE_LOGF(LogReplicationGraph, VeryVerbose, "UReplicationGraphNode_DynamicSpatialFrequency::CalcFrequencyForActor: Using viewer %ls for spatical determination for actor %ls", ((LowestDistanceViewer->Connection != nullptr) ? *LowestDistanceViewer->Connection->GetName() : TEXT("INVALID")), *Actor->GetName());

	// Skip if past cull distance
	if (!IgnoreCullDistance && ConnectionInfo.GetCullDistanceSquared() > 0.f && SmallestDistanceToActorSq > ConnectionInfo.GetCullDistanceSquared())
	{
		RemoveExistingItem();
		return;
	}

	// --------------------------------------------------------------------------------------------------------
	// Find Zone
	// --------------------------------------------------------------------------------------------------------
	const FVector& ConnectionViewDir = LowestDistanceViewer->ViewDir;
	const FVector DirToActor = (GlobalInfo.WorldLocation - LowestDistanceViewer->ViewLocation);
	const float DistanceToActor = static_cast<float>(FMath::Sqrt(SmallestDistanceToActorSq));
	const FVector NormDirToActor = DistanceToActor > SMALL_NUMBER ? (DirToActor / DistanceToActor) : DirToActor;
	const float DotP = FMath::Clamp(static_cast<float>(FVector::DotProduct(NormDirToActor, ConnectionViewDir)), -1.0f, 1.0f);

	const bool ActorSupportsFastShared = (GlobalInfo.Settings.FastSharedReplicationFunc != nullptr);
	TArrayView<FSpatializationZone>& ZoneList = ActorSupportsFastShared ? MySettings.ZoneSettings : MySettings.ZoneSettings_NonFastSharedActors;

	FGatheredReplicationActorLists DependentListContainer;
		
	for (int32 ZoneIdx=0; ZoneIdx < ZoneList.Num(); ++ZoneIdx)
	{
		FSpatializationZone& ZoneInfo = ZoneList[ZoneIdx];
		if (DotP <= ZoneInfo.MinDotProduct)
		{
			int32 FramesTillReplicate;
			bool EnableFastPath = false;
	
			// --------------------------------------------------------------------------------------------------------
			// Calc FrameTillReplicate
			// --------------------------------------------------------------------------------------------------------
			{
				// Calc Percentage of distance relative to cull distance, scaled to ZoneInfo Min/Max pct
				const float CullDistSq = ConnectionInfo.GetCullDistanceSquared() > 0.f ? ConnectionInfo.GetCullDistanceSquared() : GlobalInfo.Settings.GetCullDistanceSquared(); // Use global settings if the connection specific setting is zero'd out

				if (CullDistSq <= 0.f)
				{
					// This actor really should not be in this node
					UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_DynamicSpatialFrequency::GatherActors: %ls has cull distance of 0 (connection %f | global %f). Removing from node",
						*GetPathNameSafe(Actor), ConnectionInfo.GetCullDistanceSquared(), GlobalInfo.Settings.GetCullDistanceSquared());
					RemoveExistingItem();
					return;
				}

				const float CullDist = ConnectionInfo.GetCullDistance();
				const float DistPct = DistanceToActor / CullDist;

				const float BiasDistPct = DistPct - ZoneInfo.MinDistPct;
				const float FinalPCT = FMath::Clamp<float>( BiasDistPct / (ZoneInfo.MaxDistPct - ZoneInfo.MinDistPct), 0.f, 1.f);

				// Calc Replication period for Normal replication
				CalcDynamicReplicationPeriod(FinalPCT, ZoneInfo.MinRepPeriod, ZoneInfo.MaxRepPeriod, ConnectionInfo.ReplicationPeriodFrame, ConnectionInfo.NextReplicationFrameNum, ConnectionInfo.LastRepFrameNum, FrameNum, false);
				FramesTillReplicate = (int32)ConnectionInfo.NextReplicationFrameNum - (int32)FrameNum;

				// Update actor timeout frame here in case we get starved and can't actually replicate before then
				ConnectionInfo.ActorChannelCloseFrameNum = FMath::Max<uint32>(ConnectionInfo.ActorChannelCloseFrameNum, ConnectionInfo.NextReplicationFrameNum + 1);

				// Update the dependent actor's timeout frame too
				GlobalInfo.GatherDependentActorLists(RepGraphConnection, DependentListContainer);
				const TArrayView<const FActorRepListType> DependentActors = DependentListContainer.ViewActors(EActorRepListTypeFlags::Default);
				for (AActor* DependentActor : DependentActors)
				{
					FConnectionReplicationActorInfo& DependentActorConnectionInfo = ConnectionMap.FindOrAdd(DependentActor);
					DependentActorConnectionInfo.ActorChannelCloseFrameNum = FMath::Max<uint32>(ConnectionInfo.ActorChannelCloseFrameNum, ConnectionInfo.NextReplicationFrameNum + 1);
				}
				DependentListContainer.Reset();

				// Calc Replication Period for FastShared replication
				if (ActorSupportsFastShared && ZoneInfo.FastPath_MinRepPeriod > 0)
				{
					CalcDynamicReplicationPeriod(FinalPCT, ZoneInfo.FastPath_MinRepPeriod, ZoneInfo.FastPath_MaxRepPeriod, ConnectionInfo.FastPath_ReplicationPeriodFrame, ConnectionInfo.FastPath_NextReplicationFrameNum, ConnectionInfo.FastPath_LastRepFrameNum, FrameNum, true);
					FramesTillReplicate = FMath::Min<int32>(FramesTillReplicate, (int32)ConnectionInfo.FastPath_NextReplicationFrameNum - (int32)FrameNum);
					EnableFastPath = true;
				}

#if ENABLE_DRAW_DEBUG
				if (CVar_RepGraph_DynamicSpatialFrequency_Draw > 0 )
				{
					//CVar_RepGraph_DynamicSpatialFrequency_Draw = 0;

					static float DebugTextDuration = -1.f;

					ForEachClientPIEWorld([&](UWorld *ClientWorld)
					{
						FlushPersistentDebugLines(ClientWorld);
						FlushDebugStrings(ClientWorld);

						//FString DebugStringFull = FString::Printf(TEXT("DistanceToActor: %.2f. DistPct: %.2f. FinalPCT: %.2f. ExtraPeriod: %d. FramesTillNextReplicate: %d"), DistanceToActor, DistPct, FinalPCT, ExtraPeriod, FramesTillNextReplicate);
						FString DebugString = FString::Printf(TEXT("%.2f %.2f %d %d"), DistPct, FinalPCT, ConnectionInfo.ReplicationPeriodFrame, ConnectionInfo.FastPath_ReplicationPeriodFrame);
						DrawDebugString(ClientWorld, GlobalInfo.WorldLocation + FVector(0.f, 0.f, 50.f), DebugString, nullptr, DynamicSpatialFrequencyDebugColorArray[ZoneIdx % DynamicSpatialFrequencyDebugColorArray.Num()], DebugTextDuration, true);
					});
				}
#endif
			}

			// --------------------------------------------------------------------------------------------------------
			// We now know when this actor should replicate next. We either need to add or remove the item from the sorted list. 
			// We also may need to do some tracking for replicate this frame vs next (for Opportunistic LoadBalance)
			// --------------------------------------------------------------------------------------------------------

			if (FramesTillReplicate < 0)
			{
				// This actor is ready to go (or overdue). Add it to the replication list that we will sort
				AddOrUpdateItem(Actor, FramesTillReplicate, EnableFastPath, &GlobalInfo, &ConnectionInfo);
			}

			else if (FramesTillReplicate == 0)
			{
				// This actor is also ready to go but we may need to count it as a 'replicates this frame and not every frame' actor
				if (ReplicatesEveryFrame(ConnectionInfo, EnableFastPath) == false)
				{
					//UE_LOGF(LogReplicationGraph, Display, "  THIS[%d]: %ls. Def: %d (%d) Fast: %d (%d)", FrameNum, *Actor->GetName(), ConnectionInfo.NextReplicationFrameNum, ConnectionInfo.ReplicationPeriodFrame, ConnectionInfo.FastPath_NextReplicationFrameNum, ConnectionInfo.FastPath_ReplicationPeriodFrame);
					NumExpectedReplicationsThisFrame++; // Replicating this frame but not an 'every frame' actor
				}
					
				AddOrUpdateItem(Actor, FramesTillReplicate, EnableFastPath, &GlobalInfo, &ConnectionInfo);
			}
			else if (FramesTillReplicate == 1)
			{
				// This actors is not ready to replicate, but wants to replicate next frame.
				if (ReplicatesEveryFrame(ConnectionInfo, EnableFastPath) == false)
				{
					NumExpectedReplicationsNextFrame++; // "Not every frame" actor that expects to replicate next frame
					//UE_LOGF(LogReplicationGraph, Display, "  NEXT[%d]: %ls. Def: %d (%d) Fast: %d (%d)", FrameNum, *Actor->GetName(), ConnectionInfo.NextReplicationFrameNum, ConnectionInfo.ReplicationPeriodFrame, ConnectionInfo.FastPath_NextReplicationFrameNum, ConnectionInfo.FastPath_ReplicationPeriodFrame);
				}

				RemoveExistingItem();
			}
			else
			{
				// More than 1 frame away from replicating. Just remove it.
				RemoveExistingItem();
			}

			// This actor has been fully processed
			return;
		}
	}

	// No zone was found. This is bad
	UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_DynamicSpatialFrequency::CalcFrequencyForActor: %ls was not placed in any valid zone. Viewer: %ls DotP: %.2f ", *Actor->GetName(), ((LowestDistanceViewer->Connection != nullptr) ? *LowestDistanceViewer->Connection->GetName() : TEXT("INVALID")), DotP);
	RemoveExistingItem();
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_DynamicSpatialFrequency::GatherActors(const FActorRepListRefView& RepList, FGlobalActorReplicationInfoMap& GlobalMap, FPerConnectionActorInfoMap& ConnectionMap, const FConnectionGatherActorListParameters& Params, UNetConnection* NetConnection)
{
	UReplicationGraph* RepGraph = GraphGlobals->ReplicationGraph;
	FSettings& MySettings = GetSettings();
	const uint32 FrameNum = Params.ReplicationFrameNum;	

	const bool bReplay = NetConnection->IsReplay();

	for (AActor* Actor : RepList)
	{
		if (!bReplay)
		{
			bool bShouldSkipActor = false;
			// Don't replicate the connection view target like this. It will be done through a connection specific node
			for (const FNetViewer& CurViewer : Params.Viewers)
			{
				if (UNLIKELY(Actor == CurViewer.ViewTarget))
				{
					bShouldSkipActor = true;
					break;
				}
			}

			if (bShouldSkipActor)
			{
				continue;
			}
		}

		FGlobalActorReplicationInfo& GlobalInfo = GlobalMap.Get(Actor);

		CalcFrequencyForActor(Actor, RepGraph, Params.ConnectionManager, GlobalInfo, ConnectionMap, MySettings, Params.Viewers, FrameNum, INDEX_NONE);
	}
}

void UReplicationGraphNode_DynamicSpatialFrequency::GatherActors_DistanceOnly(const FActorRepListRefView& RepList, FGlobalActorReplicationInfoMap& GlobalMap, FPerConnectionActorInfoMap& ConnectionMap, const FConnectionGatherActorListParameters& Params)
{
	FGlobalActorReplicationInfoMap* GlobalActorReplicationInfoMap = GraphGlobals->GlobalActorReplicationInfoMap;
	for (AActor* Actor : RepList)
	{
		FGlobalActorReplicationInfo& GlobalInfo = GlobalMap.Get(Actor);
		FVector::FReal ShortestDistanceToActorSq = std::numeric_limits<FVector::FReal>::max();
		bool bShouldSkipActor = false;

		// Don't replicate the connection view target like this. It will be done through a connection specific node
		for (const FNetViewer& CurViewer : Params.Viewers)
		{
			if (UNLIKELY(Actor == CurViewer.ViewTarget))
			{
				bShouldSkipActor = true;
				break;
			}

			ShortestDistanceToActorSq = FMath::Min((GlobalInfo.WorldLocation - CurViewer.ViewLocation).SizeSquared(), ShortestDistanceToActorSq);
		}

		if (bShouldSkipActor)
		{
			continue;
		}

		SortedReplicationList.Emplace(Actor, FMath::TruncToInt32(ShortestDistanceToActorSq), &GlobalInfo);
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------

// Default value is to disable the frame-based obsolete condition
uint32 UReplicationGraphNode_ConnectionDormancyNode::NumFramesUntilObsolete = 0;

void UReplicationGraphNode_ConnectionDormancyNode::SetNumFramesUntilObsolete(uint32 InNumFrames)
{
	UE_LOGF(LogReplicationGraph, Log, "SetNumFramesUntilObsolete setting is now %u", InNumFrames);
    NumFramesUntilObsolete = InNumFrames;
}

void UReplicationGraphNode_ConnectionDormancyNode::TearDown()
{
	Super::TearDown();

	RemovedStreamingLevelActorListCollection.TearDown();
}

void UReplicationGraphNode_ConnectionDormancyNode::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
#if WITH_SERVER_CODE
	RG_QUICK_SCOPE_CYCLE_COUNTER(RepGraphNode_ConnectionDormancy_Gather);

	LastGatheredFrame = Params.ReplicationFrameNum;

	ConditionalGatherDormantActorsForConnection(ReplicationActorList, Params, nullptr);
	
	for (int32 idx=StreamingLevelCollection.StreamingLevelLists.Num()-1; idx>=0; --idx)
	{
		FStreamingLevelActorListCollection::FStreamingLevelActors& StreamingList = StreamingLevelCollection.StreamingLevelLists[idx];
		if (StreamingList.ReplicationActorList.Num() <= 0)
		{
			StreamingLevelCollection.StreamingLevelLists.RemoveAtSwap(idx, EAllowShrinking::No);
			continue;
		}

		if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
		{
			FStreamingLevelActorListCollection::FStreamingLevelActors* RemoveList = RemovedStreamingLevelActorListCollection.StreamingLevelLists.FindByKey(StreamingList.StreamingLevelName);
			if (!RemoveList)
			{
				RemoveList = &RemovedStreamingLevelActorListCollection.StreamingLevelLists.Emplace_GetRef(StreamingList.StreamingLevelName);
				Params.ConnectionManager.OnClientVisibleLevelNameAddMap.FindOrAdd(StreamingList.StreamingLevelName).AddUObject(this, &UReplicationGraphNode_ConnectionDormancyNode::OnClientVisibleLevelNameAdd);
			}

			ConditionalGatherDormantActorsForConnection(StreamingList.ReplicationActorList, Params, &RemoveList->ReplicationActorList);
		}
		else
		{
			UE_LOGF(LogReplicationGraph, Verbose, "Level Not Loaded %ls. (Client has %d levels loaded)", *StreamingList.StreamingLevelName.ToString(), Params.ClientVisibleLevelNamesRef.Num());
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_ConnectionDormancyNode::ConditionalGatherDormantActorsForConnection(FActorRepListRefView& ConnectionList, const FConnectionGatherActorListParameters& Params, FActorRepListRefView* RemovedList)
{
#if WITH_SERVER_CODE
	FPerConnectionActorInfoMap& ConnectionActorInfoMap = Params.ConnectionManager.ActorInfoMap;
	FGlobalActorReplicationInfoMap* GlobalActorReplicationInfoMap = GraphGlobals->GlobalActorReplicationInfoMap;

	// We can trickle if the TrickelStartCounter is 0. (Just trying to give it a few frames to settle)
	bool bShouldTrickle = TrickleStartCounter == 0;

	for (int32 idx = ConnectionList.Num()-1; idx >= 0; --idx)
	{
		FActorRepListType Actor = ConnectionList[idx];
		if (!ensureMsgf(IsActorValidForReplication_LogMoreInfo(Actor), TEXT("Actor not valid for replication")))
		{
			continue;
		}
		FConnectionReplicationActorInfo& ConnectionActorInfo = ConnectionActorInfoMap.FindOrAdd(Actor);
		if (ConnectionActorInfo.bDormantOnConnection)
		{
			// If we trickled this actor it doesn't need to be always relevant anymore.
			ConnectionActorInfo.bForceCullDistanceToZero = false;

			// It can be removed
			ConnectionList.RemoveAtSwap(idx);
			if (RemovedList)
			{
				RemovedList->Add(Actor);
			}

			UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "GRAPH_DORMANCY: Actor %ls is Dormant on %ls. Removing from list. (%d elements left)", *Actor->GetPathName(), *GetName(), ConnectionList.Num());
			bShouldTrickle = false; // Don't trickle this frame because we are still encountering dormant actors
		}
		else if (CVar_RepGraph_TrickleDistCullOnDormancyNodes > 0 && bShouldTrickle && ConnectionActorInfo.GetCullDistanceSquared() > 0.0f)
		{
			ConnectionActorInfo.bForceCullDistanceToZero = true;
			bShouldTrickle = false; // trickle one actor per frame
		}
	}

	if (ConnectionList.Num() > 0)
	{
		Params.OutGatheredReplicationLists.AddReplicationActorList(ConnectionList);
		
		if (TrickleStartCounter > 0)
		{
			TrickleStartCounter--;		
		}
	}
#endif // WITH_SERVER_CODE
}

bool ContainsReverse(const FActorRepListRefView& List, FActorRepListType Actor)
{
	for (int32 idx=List.Num()-1; idx >= 0; --idx)
	{
		if (List[idx] == Actor)
		{
			return true;
		}
	}

	return false;
}

void UReplicationGraphNode_ConnectionDormancyNode::NotifyActorDormancyFlush(FActorRepListType Actor)
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(ConnectionDormancyNode_NotifyActorDormancyFlush);

	FNewReplicatedActorInfo ActorInfo(Actor);

	// Dormancy is flushed so we need to make sure this actor is on this connection specific node.
	// Guard against dupes in the list. Sometimes actors flush multiple times in a row or back to back frames.
	// 
	// It may be better to track last flush frame on GlobalActorRepInfo? 
	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		if (!ContainsReverse(ReplicationActorList, Actor))
		{
			ReplicationActorList.Add(ActorInfo.Actor);
		}
	}
	else
	{
		FStreamingLevelActorListCollection::FStreamingLevelActors* Item = StreamingLevelCollection.StreamingLevelLists.FindByKey(ActorInfo.StreamingLevelName);
		if (!Item)
		{
			Item = &StreamingLevelCollection.StreamingLevelLists.Emplace_GetRef(ActorInfo.StreamingLevelName);
			Item->ReplicationActorList.Add(ActorInfo.Actor);

		} 
		else if(!ContainsReverse(Item->ReplicationActorList, Actor))
		{
			Item->ReplicationActorList.Add(ActorInfo.Actor);
		}

		// Remove from RemoveList
		FStreamingLevelActorListCollection::FStreamingLevelActors* RemoveList = RemovedStreamingLevelActorListCollection.StreamingLevelLists.FindByKey(ActorInfo.StreamingLevelName);
		if (RemoveList)
		{
			RemoveList->ReplicationActorList.RemoveFast(Actor);
		}
	}
}

void UReplicationGraphNode_ConnectionDormancyNode::OnClientVisibleLevelNameAdd(FName LevelName, UWorld* World)
{
	FStreamingLevelActorListCollection::FStreamingLevelActors* RemoveList = RemovedStreamingLevelActorListCollection.StreamingLevelLists.FindByKey(LevelName);
	if (!RemoveList)
	{
		UE_LOGF(LogReplicationGraph, Warning, ":OnClientVisibleLevelNameAdd called on %ls but there is no RemoveList. How did this get bound in the first place?. Level: %ls", *GetPathName(), *LevelName.ToString());
		return;
	}

	FStreamingLevelActorListCollection::FStreamingLevelActors* AddList = StreamingLevelCollection.StreamingLevelLists.FindByKey(LevelName);
	if (!AddList)
	{
		AddList = &StreamingLevelCollection.StreamingLevelLists.Emplace_GetRef(LevelName);
	}

	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails, LogReplicationGraph, Display, "::OnClientVisibleLevelNameadd %ls. LevelName: %ls.", *GetPathName(), *LevelName.ToString());
	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails, LogReplicationGraph, Display, "    CurrentAddList: %ls", *AddList->ReplicationActorList.BuildDebugString());
	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails, LogReplicationGraph, Display, "    RemoveList: %ls", *RemoveList->ReplicationActorList.BuildDebugString());

	AddList->ReplicationActorList.AppendContentsFrom(RemoveList->ReplicationActorList);

	RemoveList->ReplicationActorList.Reset();
}

bool UReplicationGraphNode_ConnectionDormancyNode::NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool WarnIfNotFound)
{
	QUICK_SCOPE_CYCLE_COUNTER(ConnectionDormancyNode_NotifyRemoveNetworkActor);

	// Remove from active list by calling super
	if (Super::RemoveNetworkActorFast(ActorInfo))
	{
		return true;
	}

	// Not found in active list. We must check out RemovedActorList
	return RemovedStreamingLevelActorListCollection.RemoveActorFast(ActorInfo, this);
}

bool UReplicationGraphNode_ConnectionDormancyNode::NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	bool bMovedSomething = Super::NotifyActorRenamed(ActorInfo, bWarnIfNotFound);

	const bool bRemovedSomething = RemovedStreamingLevelActorListCollection.RemoveActorFromLevelFast(ActorInfo.NewActorInfo.Actor, ActorInfo.OldActorInfo.StreamingLevelName);
	if (bRemovedSomething)
	{
		RemovedStreamingLevelActorListCollection.AddActor(ActorInfo.NewActorInfo);
	}

	return bMovedSomething || bRemovedSomething;
}

void UReplicationGraphNode_ConnectionDormancyNode::NotifyResetAllNetworkActors()
{
	Super::NotifyResetAllNetworkActors();
	RemovedStreamingLevelActorListCollection.Reset();
}

void UReplicationGraphNode_ConnectionDormancyNode::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const FStreamingLevelActorListCollection::FStreamingLevelActors& List : RemovedStreamingLevelActorListCollection.StreamingLevelLists)
	{
		for (const FActorRepListType& Actor : List.ReplicationActorList)
		{
			VerifyActorReference(Actor);
		}
	}
}

bool UReplicationGraphNode_ConnectionDormancyNode::IsNodeObsolete(uint32 CurrentFrame) const
{
	// Test if the connection tied to this node has been destroyed
	const bool bIsConnectionDestroyed = ConnectionOwner.ResolveObjectPtr() == nullptr;

	// Test if the connection has gathered the node recently.
	// After some time we can consider the client to be far enough from the node location for the node to be obsolete.
	const bool bHasBeenGatheredRecently = (NumFramesUntilObsolete == 0) || (CurrentFrame - LastGatheredFrame) <= NumFramesUntilObsolete;

	return bIsConnectionDestroyed || !bHasBeenGatheredRecently;
}

// --------------------------------------------------------------------------------------------------------------------------------------------

FVector::FReal UReplicationGraphNode_DormancyNode::MaxZForConnection = UE::Net::Private::RepGraphWorldMax;

void UReplicationGraphNode_DormancyNode::CallFunctionOnValidConnectionNodes(FConnectionDormancyNodeFunction Function)
{
	enum class ObsoleteNodeBehavior
	{
		AlwaysValid = 0, // Keep calling functions on obsolete nodes (default behavior)
		Destroy = 1,  // Destroy the nodes immediately (one time cpu hit)
	};	
	const ObsoleteNodeBehavior CurrentObsoleteNodeBehavior = (ObsoleteNodeBehavior)CVar_RepGraph_DormancyNode_ObsoleteBehavior;

	const uint32 CurrentFrame = GraphGlobals->ReplicationGraph->GetReplicationGraphFrame();

	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_DormancyNode_ConnectionLoop);
	for (FConnectionDormancyNodeMap::TIterator It = ConnectionNodes.CreateIterator(); It; ++It)
	{
		UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode = It.Value();
		const bool bIsNodeObsolete = (CurrentObsoleteNodeBehavior != ObsoleteNodeBehavior::AlwaysValid) && ConnectionNode->IsNodeObsolete(CurrentFrame);

		if (!bIsNodeObsolete)
		{
			Function(ConnectionNode);
		}
		else if (bIsNodeObsolete && CurrentObsoleteNodeBehavior == ObsoleteNodeBehavior::Destroy)
		{
			RG_QUICK_SCOPE_CYCLE_COUNTER(RepGraphNode_Dormancy_DestroyObsoleteConnectionNode);
			
			bool bWasRemoved = RemoveChildNode(ConnectionNode, UReplicationGraphNode::NodeOrdering::IgnoreOrdering);
			ensureMsgf(bWasRemoved, TEXT("DormancyNode did not find %s in it's child node."), *ConnectionNode->GetName());

			It.RemoveCurrent();
		}
	}
}

void UReplicationGraphNode_DormancyNode::NotifyResetAllNetworkActors()
{
	if (GraphGlobals.IsValid())
	{
		// Unregister dormancy callbacks first
		for (FActorRepListType& Actor : ReplicationActorList)
		{
			FGlobalActorReplicationInfo& GlobalInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(Actor);
			GlobalInfo.Events.DormancyFlush.RemoveAll(this);
		}
	}

	// Dump our global actor list
	Super::NotifyResetAllNetworkActors();

	auto ResetAllActorsFunction = [](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
		ConnectionNode->NotifyResetAllNetworkActors();
	};
	CallFunctionOnValidConnectionNodes(ResetAllActorsFunction);
}

void UReplicationGraphNode_DormancyNode::AddDormantActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo)
{
	QUICK_SCOPE_CYCLE_COUNTER(DormancyNode_AddDormantActor);
	
	Super::NotifyAddNetworkActor(ActorInfo);

	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0 && ConnectionNodes.Num() > 0, LogReplicationGraph, Display, "GRAPH_DORMANCY: AddDormantActor %ls on %ls. Adding to %d connection nodes.", *ActorInfo.Actor->GetPathName(), *GetName(), ConnectionNodes.Num());
	
	auto AddActorFunction = [ActorInfo](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
        QUICK_SCOPE_CYCLE_COUNTER(ConnectionDormancyNode_NotifyAddNetworkActor);
		ConnectionNode->NotifyAddNetworkActor(ActorInfo);
	};
	CallFunctionOnValidConnectionNodes(AddActorFunction);

	// Tell us if this item flushes net dormancy so we force it back on connection lists
	if (!GlobalInfo.Events.DormancyFlush.IsBoundToObject(this))
	{
		GlobalInfo.Events.DormancyFlush.AddUObject(this, &UReplicationGraphNode_DormancyNode::OnActorDormancyFlush);
	}
	else
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_DormancyNode already bound to dormancyflush for Actor %ls", *GetPathNameSafe(ActorInfo.GetActor()));
	}
}

void UReplicationGraphNode_DormancyNode::RemoveDormantActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo)
{
	QUICK_SCOPE_CYCLE_COUNTER(DormancyNode_RemoveDormantActor);

	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_DormancyNode::RemoveDormantActor %ls on %ls. (%d connection nodes). ChildNodes: %d", *GetNameSafe(ActorInfo.Actor), *GetPathName(), ConnectionNodes.Num(), AllChildNodes.Num());

	Super::RemoveNetworkActorFast(ActorInfo);

	ActorRepInfo.Events.DormancyFlush.RemoveAll(this);
	
	auto RemoveActorFunction = [ActorInfo](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
		// Don't warn if not found, the node may have removed the actor itself. Not worth the extra bookkeeping to skip the call.
		ConnectionNode->NotifyRemoveNetworkActor(ActorInfo, false);
	};
	CallFunctionOnValidConnectionNodes(RemoveActorFunction);
}

void UReplicationGraphNode_DormancyNode::RenameDormantActor(const FRenamedReplicatedActorInfo& ActorInfo)
{
	// Set up a FNewReplicatedActorInfo with the old level name so that the remove can find the actor
	Super::RemoveNetworkActorFast(ActorInfo.OldActorInfo);

	auto RemoveActorFunction = [ActorInfo](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
		// Don't warn if not found, the node may have removed the actor itself. Not worth the extra bookkeeping to skip the call.
		ConnectionNode->NotifyRemoveNetworkActor(ActorInfo.OldActorInfo, false);
	};
	CallFunctionOnValidConnectionNodes(RemoveActorFunction);


	// Add using new outer
	Super::NotifyAddNetworkActor(ActorInfo.NewActorInfo);

	auto AddActorFunction = [&ActorInfo](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
        QUICK_SCOPE_CYCLE_COUNTER(ConnectionDormancyNode_NotifyAddNetworkActor);
		ConnectionNode->NotifyAddNetworkActor(ActorInfo.NewActorInfo);
	};
	CallFunctionOnValidConnectionNodes(AddActorFunction);
}

void UReplicationGraphNode_DormancyNode::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
#if WITH_SERVER_CODE
	int32 NumViewersAboveMaxZ = 0;
	for (const FNetViewer& CurViewer : Params.Viewers)
	{
		if (CurViewer.ViewLocation.Z > MaxZForConnection)
		{
			++NumViewersAboveMaxZ;
		}
	}

	// If we're above max on all viewers, don't gather actors.
	if (Params.Viewers.Num() <= NumViewersAboveMaxZ)
	{
		return;
	}

	UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode = GetExistingConnectionNode(Params);

	if( ConnectionNode )
	{
		ConnectionNode->GatherActorListsForConnection(Params);
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(RepGraphNode_ConnectionDormancy_NewNodeFirstGather);
		ConnectionNode = CreateConnectionNode(Params);
		ConnectionNode->GatherActorListsForConnection(Params);
	}
#endif // WITH_SERVER_CODE
}

UReplicationGraphNode_ConnectionDormancyNode* UReplicationGraphNode_DormancyNode::GetExistingConnectionNode(const FConnectionGatherActorListParameters& Params)
{
	UReplicationGraphNode_ConnectionDormancyNode** ConnectionNodeItem = ConnectionNodes.Find(FRepGraphConnectionKey(&Params.ConnectionManager));
	return ConnectionNodeItem == nullptr ? nullptr : *ConnectionNodeItem;
}

UReplicationGraphNode_ConnectionDormancyNode* UReplicationGraphNode_DormancyNode::GetConnectionNode(const FConnectionGatherActorListParameters& Params)
{
	UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode = GetExistingConnectionNode(Params);
	
	if (ConnectionNode == nullptr)
	{
		ConnectionNode = CreateConnectionNode(Params);
	}

	return ConnectionNode;
}

UReplicationGraphNode_ConnectionDormancyNode* UReplicationGraphNode_DormancyNode::CreateConnectionNode(const FConnectionGatherActorListParameters& Params)
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(RepGraphNode_Dormancy_Create_ConnectionNode);
	
	FRepGraphConnectionKey RepGraphConnection(&Params.ConnectionManager);

	// We do not have a per-connection node for this connection, so create one and copy over contents
	UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode = CreateChildNode<UReplicationGraphNode_ConnectionDormancyNode>();
	ConnectionNodes.Add(RepGraphConnection) = ConnectionNode;

	// Copy our main lists to the connection node
	ConnectionNode->DeepCopyActorListsFrom(this);

	ConnectionNode->InitConnectionNode(RepGraphConnection, Params.ReplicationFrameNum);

	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0, LogReplicationGraph, Display, "GRAPH_DORMANCY: First time seeing connection %ls in node %ls. Created ConnectionDormancyNode %ls.", *Params.ConnectionManager.GetName(), *GetName(), *ConnectionNode->GetName());

	return ConnectionNode;
}

void UReplicationGraphNode_DormancyNode::OnActorDormancyFlush(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo)
{
	QUICK_SCOPE_CYCLE_COUNTER(DormancyNode_OnActorDormancyFlush);

	if (CVar_RepGraph_Verify)
	{
		FNewReplicatedActorInfo ActorInfo(Actor);
		if (ActorInfo.StreamingLevelName == NAME_None)
		{
			ensureMsgf(ReplicationActorList.Contains(Actor), TEXT("UReplicationGraphNode_DormancyNode::OnActorDormancyFlush %s not present in %s actor lists!"), *Actor->GetPathName(), *GetPathName());
		}
		else
		{
			if (FStreamingLevelActorListCollection::FStreamingLevelActors* Item = StreamingLevelCollection.StreamingLevelLists.FindByKey(ActorInfo.StreamingLevelName))
			{
				ensureMsgf(Item->ReplicationActorList.Contains(Actor), TEXT("UReplicationGraphNode_DormancyNode::OnActorDormancyFlush %s not present in %s actor lists! Streaming Level: %s"), *GetActorRepListTypeDebugString(Actor), *GetPathName(), *ActorInfo.StreamingLevelName.ToString());
			}
		}
	}

	// -------------------
		
	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails > 0 && ConnectionNodes.Num() > 0, LogReplicationGraph, Display, "GRAPH_DORMANCY: Actor %ls Flushed Dormancy. %ls. Refreshing all %d connection nodes.", *Actor->GetPathName(), *GetName(), ConnectionNodes.Num());

	auto DormancyFlushFunction = [Actor](UReplicationGraphNode_ConnectionDormancyNode* ConnectionNode)
	{
		ConnectionNode->NotifyActorDormancyFlush(Actor);
	};
	CallFunctionOnValidConnectionNodes(DormancyFlushFunction);
}

void UReplicationGraphNode_DormancyNode::ConditionalGatherDormantDynamicActors(FActorRepListRefView& RepList, const FConnectionGatherActorListParameters& Params, FActorRepListRefView* RemovedList, bool bEnforceReplistUniqueness /* = false */, FActorRepListRefView* RemoveFromList /* = nullptr */)
{
	auto GatherDormantDynamicActorsForList = [&](const FActorRepListRefView& InReplicationActorList)
	{
		RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_DormancyNode_ConditionalGatherDormantDynamicActors);
		for (const FActorRepListType& Actor : InReplicationActorList)
		{
			if (Actor && !Actor->IsNetStartupActor())
			{
				if (FConnectionReplicationActorInfo* Info = Params.ConnectionManager.ActorInfoMap.Find(Actor))
				{
					// Need to grab the actors with valid channels here as they will be going dormant soon, but might not be quite yet
					if (Info->bDormantOnConnection || Info->Channel != nullptr)
					{
						if (RemovedList && RemovedList->Contains(Actor))
						{
							continue;
						}

						// Keeping the allocation when removing items let's us safe time on multiple items removed from that list.
						if (RemoveFromList && RemoveFromList->RemoveFast(Actor, EAllowShrinking::No))
						{
							Info->bGridSpatilization_AlreadyDormant = false;
						}

						// Prevent adding actors if we already have added them, this saves on grow operations.
						if (bEnforceReplistUniqueness)
						{
							if (Info->bGridSpatilization_AlreadyDormant)
							{
								continue;
							}
							else
							{
								Info->bGridSpatilization_AlreadyDormant = true;
							}
						}

						RepList.ConditionalAdd(Actor);
					}
				}
			}
		}
	};

	GatherDormantDynamicActorsForList(ReplicationActorList);

	for (const FStreamingLevelActorListCollection::FStreamingLevelActors& StreamingList : StreamingLevelCollection.StreamingLevelLists)
	{
		if (Params.CheckClientVisibilityForLevel(StreamingList.StreamingLevelName))
		{
			GatherDormantDynamicActorsForList(StreamingList.ReplicationActorList);
		}
	}
}

// --------------------------------------------------------------------------------------------------------------------------------------------


void UReplicationGraphNode_GridCell::AddStaticActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bParentNodeHandlesDormancyChange)
{
	if (ActorRepInfo.bWantsToBeDormant)
	{
		// Pass to dormancy node
		GetDormancyNode()->AddDormantActor(ActorInfo, ActorRepInfo);
	}
	else
	{	
		// Put it in our non dormancy list
		Super::NotifyAddNetworkActor(ActorInfo);
	}

	// We need to be told if this actor changes dormancy so we can move it between nodes. Unless our parent is going to do it.
	if (!bParentNodeHandlesDormancyChange)
	{
		ActorRepInfo.Events.DormancyChange.AddUObject(this, &UReplicationGraphNode_GridCell::OnStaticActorNetDormancyChange);
	}
}

void UReplicationGraphNode_GridCell::AddDynamicActor(const FNewReplicatedActorInfo& ActorInfo)
{
	GetDynamicNode()->NotifyAddNetworkActor(ActorInfo);
}

void UReplicationGraphNode_GridCell::RemoveStaticActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bWasAddedAsDormantActor)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_Simple2DSpatializationLeaf::RemoveStaticActor %ls on %ls", *ActorInfo.Actor->GetPathName(), *GetPathName());

	if (bWasAddedAsDormantActor)
	{
		GetDormancyNode()->RemoveDormantActor(ActorInfo, ActorRepInfo);
	}
	else
	{	
		Super::NotifyRemoveNetworkActor(ActorInfo);
	}
	
	ActorRepInfo.Events.DormancyChange.RemoveAll(this);
}

void UReplicationGraphNode_GridCell::RemoveDynamicActor(const FNewReplicatedActorInfo& ActorInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_Simple2DSpatializationLeaf::RemoveDynamicActor %ls on %ls", *ActorInfo.Actor->GetPathName(), *GetPathName());

	GetDynamicNode()->NotifyRemoveNetworkActor(ActorInfo);
}

void UReplicationGraphNode_GridCell::RenameStaticActor(const FRenamedReplicatedActorInfo& ActorInfo, bool bWasAddedAsDormantActor)
{
	if (bWasAddedAsDormantActor)
	{
		GetDormancyNode()->RenameDormantActor(ActorInfo);
	}
	else
	{
		const bool bRemovedSomething = Super::NotifyRemoveNetworkActor(ActorInfo.OldActorInfo, true);
		if (ensureMsgf(bRemovedSomething, TEXT("Renamed static actor %s not found in non-dormant grid cell list."), *GetFullNameSafe(ActorInfo.OldActorInfo.GetActor())))
		{
			Super::NotifyAddNetworkActor(ActorInfo.NewActorInfo);
		}
	}
}

void UReplicationGraphNode_GridCell::RenameDynamicActor(const FRenamedReplicatedActorInfo& ActorInfo)
{
	GetDynamicNode()->NotifyActorRenamed(ActorInfo);
}

void UReplicationGraphNode_GridCell::ConditionalCopyDormantActors(FActorRepListRefView& FromList, UReplicationGraphNode_DormancyNode* ToNode)
{
	if (GraphGlobals.IsValid())
	{
		for (int32 idx = FromList.Num()-1; idx >= 0; --idx)
		{
			FActorRepListType Actor = FromList[idx];
			FGlobalActorReplicationInfo& GlobalInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(Actor);
			if (GlobalInfo.bWantsToBeDormant)
			{
				ToNode->NotifyAddNetworkActor(FNewReplicatedActorInfo(Actor));
				FromList.RemoveAtSwap(idx);
			}
		}
	}
}

void UReplicationGraphNode_GridCell::OnStaticActorNetDormancyChange(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo, ENetDormancy NewValue, ENetDormancy OldValue)
{
	UE_CLOGF(CVar_RepGraph_LogNetDormancyDetails>0, LogReplicationGraph, Display, "UReplicationGraphNode_GridCell::OnNetDormancyChange. %ls on %ls. Old: %d, New: %d", *Actor->GetPathName(), *GetPathName(), NewValue, OldValue);

	const bool bCurrentDormant = NewValue > DORM_Awake;
	const bool bPreviousDormant = OldValue > DORM_Awake;

	if (!bCurrentDormant && bPreviousDormant)
	{
		// Actor is now awake, remove from dormancy node and add to non dormancy list
		FNewReplicatedActorInfo ActorInfo(Actor);
		GetDormancyNode()->RemoveDormantActor(ActorInfo, GlobalInfo);
		Super::NotifyAddNetworkActor(ActorInfo);
	}
	else if (bCurrentDormant && !bPreviousDormant)
	{
		// Actor is now dormant, remove from non dormant list, add to dormant node
		FNewReplicatedActorInfo ActorInfo(Actor);
		Super::NotifyRemoveNetworkActor(ActorInfo);
		GetDormancyNode()->AddDormantActor(ActorInfo, GlobalInfo);
	}
}


UReplicationGraphNode* UReplicationGraphNode_GridCell::GetDynamicNode()
{
	if (DynamicNode == nullptr)
	{
		if (CreateDynamicNodeOverride)
		{
			DynamicNode = CreateDynamicNodeOverride(this);
		}
		else
		{
			DynamicNode = CreateChildNode<UReplicationGraphNode_ActorListFrequencyBuckets>();
		}
	}

	return DynamicNode;
}

UReplicationGraphNode_DormancyNode* UReplicationGraphNode_GridCell::GetDormancyNode(bool bInCreateIfMissing)
{
	if (DormancyNode == nullptr && bInCreateIfMissing)
	{
		DormancyNode = CreateChildNode<UReplicationGraphNode_DormancyNode>();
	}

	return DormancyNode;
}

void UReplicationGraphNode_GridCell::GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const
{
	Super::GetAllActorsInNode_Debugging(OutArray);
}

int32 CVar_RepGraph_DebugNextNewActor = 0;
static FAutoConsoleVariableRef CVarRepGraphDebugNextActor(TEXT("Net.RepGraph.Spatial.DebugNextNewActor"), CVar_RepGraph_DebugNextNewActor, TEXT(""), ECVF_Default );

// -------------------------------------------------------

UReplicationGraphNode_GridSpatialization2D::UReplicationGraphNode_GridSpatialization2D()
	: CellSize(0.f)
	, SpatialBias(ForceInitToZero)
	, ConnectionMaxZ(UE::Net::Private::RepGraphWorldMax)
	, GridBounds(ForceInitToZero)
{
	bRequiresPrepareForReplicationCall = true;
	bDestroyDormantDynamicActors = CVar_RepGraph_GridSpatialization2D_DestroyDormantDynamicActorsDefault != 0;
	DestroyDormantDynamicActorsCellTTL = CVar_RepGraph_DormantCellsTTLDefault;
	ReplicatedDormantDestructionInfosPerFrame = CVar_RepGraph_ReplicatedDormantDestructionInfosPerFrame;
}

void UReplicationGraphNode_GridSpatialization2D::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraphNode_GridSpatialization2D::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ClassRebuildDenyList", ClassRebuildDenyList.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DynamicSpatializedActors", DynamicSpatializedActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("StaticSpatializedActors", StaticSpatializedActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingStaticSpatializedActors", PendingStaticSpatializedActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Grid", Grid.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GatheredNodes", GatheredNodes.CountBytes(Ar));
	}
}

void UReplicationGraphNode_GridSpatialization2D::NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo)
{
	ensureAlwaysMsgf(false, TEXT("UReplicationGraphNode_GridSpatialization2D::NotifyAddNetworkActor should not be called directly"));
}

bool UReplicationGraphNode_GridSpatialization2D::NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	ensureAlwaysMsgf(false, TEXT("UReplicationGraphNode_GridSpatialization2D::NotifyRemoveNetworkActor should not be called directly"));
	return false;
}

bool UReplicationGraphNode_GridSpatialization2D::NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound)
{
	ensureAlwaysMsgf(false, TEXT("UReplicationGraphNode_GridSpatialization2D::NotifyActorRenamed should not be called directly"));
	return false;
}

void UReplicationGraphNode_GridSpatialization2D::AddActor_Dormancy(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_GridSpatialization2D::AddActor_Dormancy %ls on %ls", *ActorInfo.Actor->GetFullName(), *GetPathName());

	if (ActorRepInfo.bWantsToBeDormant)
	{
		AddActorInternal_Static(ActorInfo, ActorRepInfo, true);
	}
	else
	{
		AddActorInternal_Dynamic(ActorInfo);
	}

	// Tell us if dormancy changes for this actor because then we need to move it. Note we don't care about Flushing.
	ActorRepInfo.Events.DormancyChange.AddUObject(this, &UReplicationGraphNode_GridSpatialization2D::OnNetDormancyChange);
}

void UReplicationGraphNode_GridSpatialization2D::RemoveActor_Static(const FNewReplicatedActorInfo& ActorInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_GridSpatialization2D::RemoveActor_Static %ls on %ls", *ActorInfo.Actor->GetFullName(), *GetPathName());

	if (GraphGlobals.IsValid())
	{
		FGlobalActorReplicationInfo* GlobalInfo = GraphGlobals->GlobalActorReplicationInfoMap->Find(ActorInfo.Actor);
		if (!GlobalInfo)
		{
			return;
		}

		RemoveActorInternal_Static(ActorInfo, *GlobalInfo, GlobalInfo->bWantsToBeDormant); 
	}
}

void UReplicationGraphNode_GridSpatialization2D::RemoveActor_Dormancy(const FNewReplicatedActorInfo& ActorInfo)
{
	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraphNode_GridSpatialization2D::RemoveActor_Dormancy %ls on %ls", *ActorInfo.Actor->GetFullName(), *GetPathName());

	if (GraphGlobals.IsValid())
	{
		FGlobalActorReplicationInfo* ActorRepInfo = GraphGlobals->GlobalActorReplicationInfoMap->Find(ActorInfo.Actor);
		if (!ActorRepInfo)
		{
			return;
		}

		if (ActorRepInfo->bWantsToBeDormant)
		{
			RemoveActorInternal_Static(ActorInfo, *ActorRepInfo, true);
		}
		else
		{
			RemoveActorInternal_Dynamic(ActorInfo);
		}

		// AddActorInternal_Static and AddActorInternal_Dynamic will both override Actor information if they are called repeatedly.
		// This means that even if AddActor_Dormancy is called multiple times with the same Actor, a single call to RemoveActor_Dormancy
		// will completely remove the Actor from either the Static or Dynamic list appropriately.
		// Therefore, it should be safe to call RemoveAll and not worry about trying to track individual delegate handles.
		ActorRepInfo->Events.DormancyChange.RemoveAll(this);
	}
}

void UReplicationGraphNode_GridSpatialization2D::RenameActor_Dormancy(const FRenamedReplicatedActorInfo& ActorInfo)
{
	if (GraphGlobals.IsValid())
	{
		FGlobalActorReplicationInfo& ActorRepInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(ActorInfo.NewActorInfo.Actor);
		if (ActorRepInfo.bWantsToBeDormant)
		{
			RenameActor_Static(ActorInfo);
		}
		else
		{
			RenameActor_Dynamic(ActorInfo);
		}
	}
}

void UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Dynamic(const FNewReplicatedActorInfo& ActorInfo)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (ActorInfo.Actor->bAlwaysRelevant)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Always relevant actor being added to spatialized graph node. %ls", *GetNameSafe(ActorInfo.Actor));
		return;
	}
#endif

	UE_CLOGF(CVar_RepGraph_LogActorRemove>0, LogReplicationGraph, Display, "UReplicationGraph::AddActorInternal_Dynamic %ls", *ActorInfo.Actor->GetFullName());

	DynamicSpatializedActors.Emplace(ActorInfo.Actor, ActorInfo);
}

void UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Static(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bDormancyDriven)
{
	AActor* Actor = ActorInfo.Actor;
	if (Actor->IsActorInitialized() == false)
	{
		// Make sure its not already in the list. This should really not happen but would be very bad if it did. This list should always be small so doing the safety check seems good.
		for (int32 idx=PendingStaticSpatializedActors.Num()-1; idx >= 0; --idx)
		{
			if (PendingStaticSpatializedActors[idx].Actor == ActorInfo.Actor)
			{
				UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Static was called on %ls when it was already in the PendingStaticSpatializedActors list!", *Actor->GetPathName());
				return;
			}
		}

		PendingStaticSpatializedActors.Emplace(ActorInfo.Actor, bDormancyDriven);
		return;
	}

	if (CVar_RepGraph_Verify)
	{
		ensureMsgf(PendingStaticSpatializedActors.Contains(ActorInfo.Actor) == false, TEXT("UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Static was called on %s when it was already in the PendingStaticSpatializedActors list!"), *Actor->GetPathName());
	}

	AddActorInternal_Static_Implementation(ActorInfo, ActorRepInfo, bDormancyDriven);
}

void UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Static_Implementation(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bDormancyDriven)
{
	AActor* Actor = ActorInfo.Actor;
	const FVector Location3D = Actor->GetActorLocation();
	ActorRepInfo.WorldLocation = Location3D;

	if (CVar_RepGraph_LogActorAdd)
	{
		UE_LOGF(LogReplicationGraph, Display, "UReplicationGraphNode_GridSpatialization2D::AddActorInternal_Static placing %ls into static grid at %ls", *Actor->GetPathName(), *ActorRepInfo.WorldLocation.ToString());
	}
		
	if (WillActorLocationGrowSpatialBounds(Location3D))
	{
		HandleActorOutOfSpatialBounds(Actor, Location3D, true);
	}

	StaticSpatializedActors.Emplace(Actor, FCachedStaticActorInfo(ActorInfo, bDormancyDriven));

	// Only put in cell right now if we aren't needing to rebuild the whole grid
	if (!bNeedsRebuild)
	{
		PutStaticActorIntoCell(ActorInfo, ActorRepInfo, bDormancyDriven);
	}
}

void UReplicationGraphNode_GridSpatialization2D::RemoveActorInternal_Dynamic(const FNewReplicatedActorInfo& ActorInfo)
{
	if (FCachedDynamicActorInfo* DynamicActorInfo = DynamicSpatializedActors.Find(ActorInfo.Actor))
	{
		if (DynamicActorInfo->CellInfo.IsValid())
		{
			GetGridNodesForActor(ActorInfo.Actor, DynamicActorInfo->CellInfo, GatheredNodes);
			for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
			{
				Node->RemoveDynamicActor(ActorInfo);
			}
		}
		DynamicSpatializedActors.Remove(ActorInfo.Actor);
	}
	else
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_Simple2DSpatialization::RemoveActorInternal_Dynamic attempted remove %ls from streaming dynamic list but it was not there.", *GetActorRepListTypeDebugString(ActorInfo.Actor));
		if (StaticSpatializedActors.Remove(ActorInfo.Actor) > 0)
		{
			UE_LOGF(LogReplicationGraph, Warning, "   It was in StaticSpatializedActors!");
		}
	}
}

void UReplicationGraphNode_GridSpatialization2D::RemoveActorInternal_Static(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bWasAddedAsDormantActor)
{
	if (StaticSpatializedActors.Remove(ActorInfo.Actor) <= 0)
	{
		// May have been a pending actor
		for (int32 idx=PendingStaticSpatializedActors.Num()-1; idx >= 0; --idx)
		{
			if (PendingStaticSpatializedActors[idx].Actor == ActorInfo.Actor)
			{
				PendingStaticSpatializedActors.RemoveAtSwap(idx, EAllowShrinking::No);
				return;
			}
		}

		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_Simple2DSpatialization::RemoveActorInternal_Static attempted remove %ls from static list but it was not there.", *GetActorRepListTypeDebugString(ActorInfo.Actor));
		if(DynamicSpatializedActors.Remove(ActorInfo.Actor) > 0)
		{
			UE_LOGF(LogReplicationGraph, Warning, "   It was in DynamicStreamingSpatializedActors!");
		}
	}

	// Remove it from the actual node it should still be in. Note that even if the actor did move in between this and the last replication frame, the FGlobalActorReplicationInfo would not have been updated
	GetGridNodesForActor(ActorInfo.Actor, ActorRepInfo, GatheredNodes);
	for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
	{
		Node->RemoveStaticActor(ActorInfo, ActorRepInfo, bWasAddedAsDormantActor);
	}

	if (CVar_RepGraph_Verify)
	{
		// Verify this actor is in no nodes. This is pretty slow!
		TArray<FActorRepListType> AllActors;
		for (auto& InnerArray : Grid)
		{
			for (UReplicationGraphNode_GridCell* N : InnerArray)
			{
				if (N)
				{
					AllActors.Reset();
					N->GetAllActorsInNode_Debugging(AllActors);
					
					ensureMsgf(AllActors.Contains(ActorInfo.Actor) == false, TEXT("Actor still in a node after removal!. %s. Removal Location: %s"), *N->GetPathName(), *ActorRepInfo.WorldLocation.ToString());
				}
			}
		}
	}
}

void UReplicationGraphNode_GridSpatialization2D::RenameActor_Static(const FRenamedReplicatedActorInfo& ActorInfo)
{
	FCachedStaticActorInfo* StaticFoundInfo = StaticSpatializedActors.Find(ActorInfo.NewActorInfo.Actor);
	if (StaticFoundInfo)
	{
		StaticFoundInfo->ActorInfo.StreamingLevelName = ActorInfo.NewActorInfo.StreamingLevelName;
	}
	else
	{
		// May have been a pending actor
		for (const FPendingStaticActors& Pending : PendingStaticSpatializedActors)
		{
			if (Pending.Actor == ActorInfo.OldActorInfo.Actor)
			{
				// Pending static actors don't store any level/outer information, so there's nothing to do
				return;
			}
		}

		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::RenameActor_Static attempted rename %ls from static list but it was not there.", *GetActorRepListTypeDebugString(ActorInfo.NewActorInfo.Actor));
		FCachedDynamicActorInfo* DynamicFoundInfo = DynamicSpatializedActors.Find(ActorInfo.NewActorInfo.Actor);
		if (DynamicFoundInfo)
		{
			DynamicFoundInfo->ActorInfo.StreamingLevelName = ActorInfo.NewActorInfo.StreamingLevelName;
			UE_LOGF(LogReplicationGraph, Warning, "   It was in DynamicSpatializedActors!");
		}
	}

	// Remove it from the actual node it should still be in. Note that even if the actor did move in between this and the last replication frame, the FGlobalActorReplicationInfo would not have been updated
	FGlobalActorReplicationInfo& GlobalInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(ActorInfo.NewActorInfo.Actor);
	GetGridNodesForActor(ActorInfo.NewActorInfo.Actor, GlobalInfo, GatheredNodes);
	for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
	{
		Node->RenameStaticActor(ActorInfo, GlobalInfo.bWantsToBeDormant);
	}
}

void UReplicationGraphNode_GridSpatialization2D::RenameActor_Dynamic(const FRenamedReplicatedActorInfo& ActorInfo)
{
	FCachedDynamicActorInfo* DynamicFoundInfo = DynamicSpatializedActors.Find(ActorInfo.NewActorInfo.Actor);
	if (DynamicFoundInfo)
	{
		if (DynamicFoundInfo->CellInfo.IsValid())
		{
			GetGridNodesForActor(ActorInfo.NewActorInfo.Actor, DynamicFoundInfo->CellInfo, GatheredNodes);
			for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
			{
				Node->RenameDynamicActor(ActorInfo);
			}
		}
		DynamicFoundInfo->ActorInfo.StreamingLevelName = ActorInfo.NewActorInfo.StreamingLevelName;
	}
	else
	{
		UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::RenameActor_Dynamic attempted rename %ls from streaming dynamic list but it was not there.", *GetActorRepListTypeDebugString(ActorInfo.NewActorInfo.Actor));
		
		FCachedStaticActorInfo* StaticFoundInfo = StaticSpatializedActors.Find(ActorInfo.NewActorInfo.Actor);
		if (StaticFoundInfo)
		{
			StaticFoundInfo->ActorInfo.StreamingLevelName = ActorInfo.NewActorInfo.StreamingLevelName;
			UE_LOGF(LogReplicationGraph, Warning, "   It was in StaticSpatializedActors!");
		}
	}
}

void UReplicationGraphNode_GridSpatialization2D::OnNetDormancyChange(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo, ENetDormancy NewValue, ENetDormancy OldValue)
{
	const bool bCurrentShouldBeStatic = NewValue > DORM_Awake;
	const bool bPreviousShouldBeStatic = OldValue > DORM_Awake;

	if (bCurrentShouldBeStatic && !bPreviousShouldBeStatic)
	{
		// Actor was dynamic and is now static. Remove from dynamic list and add to static.
		FNewReplicatedActorInfo ActorInfo(Actor);
		RemoveActorInternal_Dynamic(ActorInfo);
		AddActorInternal_Static(ActorInfo, GlobalInfo, true);
	}
	else if (!bCurrentShouldBeStatic && bPreviousShouldBeStatic)
	{
		FNewReplicatedActorInfo ActorInfo(Actor);
		RemoveActorInternal_Static(ActorInfo, GlobalInfo, true); // This is why we need the 3rd bool parameter: this actor was placed as dormant (and it no longer is at the moment of this callback)
		AddActorInternal_Dynamic(ActorInfo);
	}
}

void UReplicationGraphNode_GridSpatialization2D::NotifyResetAllNetworkActors()
{
	StaticSpatializedActors.Reset();
	DynamicSpatializedActors.Reset();
	PendingStaticSpatializedActors.Reset();
	Super::NotifyResetAllNetworkActors();
}

void UReplicationGraphNode_GridSpatialization2D::PutStaticActorIntoCell(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bDormancyDriven)
{
	GetGridNodesForActor(ActorInfo.Actor, ActorRepInfo, GatheredNodes);
	for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
	{
		Node->AddStaticActor(ActorInfo, ActorRepInfo, bDormancyDriven);
	}
}

void UReplicationGraphNode_GridSpatialization2D::GetGridNodesForActor(FActorRepListType Actor, const FGlobalActorReplicationInfo& ActorRepInfo, TArray<UReplicationGraphNode_GridCell*>& OutNodes)
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_GetGridNodesForActor);
	GetGridNodesForActor(Actor, GetCellInfoForActor(Actor, ActorRepInfo.WorldLocation, ActorRepInfo.Settings.GetCullDistance()), OutNodes);
}

void UReplicationGraphNode_GridSpatialization2D::SetBiasAndGridBounds(const FBox& GridBox)
{
	using namespace UE::Net::Private;

	const FVector2D BoxMin2D = FVector2D(GridBox.Min);
	const FVector2D BoxMax2D = FVector2D(GridBox.Max);
	
	SpatialBias = BoxMin2D;
	GridBounds = FBox( FVector(BoxMin2D, -RepGraphHalfWorldMax), FVector(BoxMax2D, RepGraphHalfWorldMax) );
}

UReplicationGraphNode_GridSpatialization2D::FActorCellInfo UReplicationGraphNode_GridSpatialization2D::GetCellInfoForActor(FActorRepListType Actor, const FVector& Location3D, float CullDistance)
{
	using namespace UE::Net::Private;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CullDistance <= 0.f)
	{
		UE_LOGF(LogReplicationGraph, Warning, "::GetGridNodesForActor called on %ls when its CullDistance = %.2f. (Must be > 0)", *GetActorRepListTypeDebugString(Actor), CullDistance);
	}
#endif

	FVector ClampedLocation = Location3D;

	// Sanity check the actor's location. If it's garbage, we could end up with a gigantic allocation in GetGridNodesForActor as we adjust the grid.
	if (Location3D.X < -RepGraphHalfWorldMax || Location3D.X > RepGraphHalfWorldMax ||
		Location3D.Y < -RepGraphHalfWorldMax || Location3D.Y > RepGraphHalfWorldMax ||
		Location3D.Z < -RepGraphHalfWorldMax || Location3D.Z > RepGraphHalfWorldMax)
	{
		FGlobalActorReplicationInfo& ActorRepInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(Actor);
		if (!ActorRepInfo.bWasWorldLocClamped)
		{
			UE_LOGF(LogReplicationGraph, Warning, "GetCellInfoForActor: Actor %ls is outside world bounds with a location of %ls. Clamping grid location to world bounds.", *GetFullNameSafe(Actor), *Location3D.ToString());	
			ActorRepInfo.bWasWorldLocClamped = true;
		}
		ClampedLocation = Location3D.BoundToCube(RepGraphHalfWorldMax);
	}

	if (!ensureMsgf(!ClampedLocation.ContainsNaN(), TEXT("GetCellInfoForActor: Actor %s has an invalid location of %s, defaulting to the origin."), *GetFullNameSafe(Actor), *ClampedLocation.ToString()))
	{
		ClampedLocation = FVector::ZeroVector;
	}

	FActorCellInfo CellInfo;
	const FVector::FReal LocationBiasX = (ClampedLocation.X - SpatialBias.X);
	const FVector::FReal LocationBiasY = (ClampedLocation.Y - SpatialBias.Y);

	const FVector::FReal Dist = CullDistance;
	const FVector::FReal MinX = FMath::Max(0, LocationBiasX - Dist);
	const FVector::FReal MinY = FMath::Max(0, LocationBiasY - Dist);
	FVector::FReal MaxX = LocationBiasX + Dist;
	FVector::FReal MaxY = LocationBiasY + Dist;

	if (GridBounds.IsValid)
	{
		const FVector BoundSize = GridBounds.GetSize();
		MaxX = FMath::Min(MaxX, BoundSize.X);
		MaxY = FMath::Min(MaxY, BoundSize.Y);
	}

	CellInfo.StartX = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>(MinX / CellSize));
	CellInfo.StartY = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>(MinY / CellSize));

	CellInfo.EndX = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>(MaxX / CellSize));
	CellInfo.EndY = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>(MaxY / CellSize));
	return CellInfo;
}

void UReplicationGraphNode_GridSpatialization2D::GetGridNodesForActor(FActorRepListType Actor, const UReplicationGraphNode_GridSpatialization2D::FActorCellInfo& CellInfo, TArray<UReplicationGraphNode_GridCell*>& OutNodes)
{
	if (!ensure(CellInfo.IsValid()))
	{
		return;
	}

	OutNodes.Reset();

	const int32 StartX = CellInfo.StartX;
	const int32 StartY = CellInfo.StartY;
	const int32 EndX = CellInfo.EndX;
	const int32 EndY = CellInfo.EndY;

	if (Grid.Num() <= EndX)
	{
		Grid.SetNum(EndX+1);
	}

	for (int32 X = StartX; X <= EndX; X++)
	{
		TArray<UReplicationGraphNode_GridCell*>& GridY = Grid[X];
		if (GridY.Num() <= EndY)
		{
			GridY.SetNum(EndY+1);
		}

		for (int32 Y = StartY; Y <= EndY; Y++)
		{
			UReplicationGraphNode_GridCell* NodePtr = GetCellNode(GridY[Y]);
			OutNodes.Add(NodePtr);
		}
	}

/*
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DebugActorNames.Num() > 0)
	{
		if (DebugActorNames.ContainsByPredicate([&](const FString DebugName) { return Actor->GetName().Contains(DebugName); }))
		{
			UE_LOGF(LogReplicationGraph, Display, "Adding Actor %ls. WorldLocation (Cached): %ls. WorldLocation (AActor): %ls. Buckets: %d/%d. SpatialBias: %ls", *Actor->GetName(), *ActorRepInfo.WorldLocation.ToString(), *Actor->GetActorLocation().ToString(), BucketX, BucketY, *SpatialBias.ToString());
		}
	}
#endif
*/
}

bool UReplicationGraphNode_GridSpatialization2D::WillActorLocationGrowSpatialBounds(const FVector& Location) const
{
	// When bounds are set, we don't grow the cells but instead clamp the actor to the bounds.
	return GridBounds.IsValid ? false : (SpatialBias.X > Location.X || SpatialBias.Y > Location.Y);
}

void UReplicationGraphNode_GridSpatialization2D::HandleActorOutOfSpatialBounds(AActor* Actor, const FVector& Location3D, const bool bStaticActor)
{
	// Don't rebuild spatialization for denied actors. They will just get clamped to the grid.
	if (ClassRebuildDenyList.Get(Actor->GetClass()) != nullptr)
	{
		return;
	}

	const bool bOldNeedRebuild = bNeedsRebuild;
	if (SpatialBias.X > Location3D.X)
	{
		bNeedsRebuild = true;
		SpatialBias.X = Location3D.X - (CellSize / 2.f);
	}
	if (SpatialBias.Y > Location3D.Y)
	{
		bNeedsRebuild = true;
		SpatialBias.Y = Location3D.Y - (CellSize / 2.f);
	}

	if (bNeedsRebuild && !bOldNeedRebuild)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Spatialization Rebuild caused by: %ls at %ls. New Bias: %ls. IsStatic: %d", *Actor->GetPathName(), *Location3D.ToString(), *SpatialBias.ToString(), (int32)bStaticActor);
	}
}

int32 CVar_RepGraph_Spatial_PauseDynamic = 0;
static FAutoConsoleVariableRef CVarRepSpatialPauseDynamic(TEXT("Net.RepGraph.Spatial.PauseDynamic"), CVar_RepGraph_Spatial_PauseDynamic, TEXT("Pauses updating dynamic actor positions in the spatialization nodes."), ECVF_Default );

int32 CVar_RepGraph_Spatial_DebugDynamic = 0;
static FAutoConsoleVariableRef CVarRepGraphSpatialDebugDynamic(TEXT("Net.RepGraph.Spatial.DebugDynamic"), CVar_RepGraph_Spatial_DebugDynamic, TEXT("Prints debug info whenever dynamic actors changes spatial cells"), ECVF_Default );

float CVar_RepGraph_Spatial_BiasCreep = 0.f;
static FAutoConsoleVariableRef CVarRepGraphSpatialBiasCreep(TEXT("Net.RepGraph.Spatial.BiasCreep"), CVar_RepGraph_Spatial_BiasCreep, TEXT("Changes bias each frame by this much and force rebuld. For stress test debugging"), ECVF_Default );

void UReplicationGraphNode_GridSpatialization2D::PrepareForReplication()
{
#if	WITH_SERVER_CODE
	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_PrepareForReplication);

	FGlobalActorReplicationInfoMap* GlobalRepMap = GraphGlobals.IsValid() ? GraphGlobals->GlobalActorReplicationInfoMap : nullptr;
	check(GlobalRepMap);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVar_RepGraph_Spatial_BiasCreep != 0.f)
	{
		SpatialBias.X += CVar_RepGraph_Spatial_BiasCreep;
		SpatialBias.Y += CVar_RepGraph_Spatial_BiasCreep;
		bNeedsRebuild = true;
	}

	// -------------------------------------------
	//	Update dynamic actors
	// -------------------------------------------
	if (CVar_RepGraph_Spatial_PauseDynamic == 0)
#endif
	{
		RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_BuildDynamic);

		for (auto& MapIt : DynamicSpatializedActors)
		{
			FActorRepListType& DynamicActor = MapIt.Key;
			FCachedDynamicActorInfo& DynamicActorInfo = MapIt.Value;
			FActorCellInfo& PreviousCellInfo = DynamicActorInfo.CellInfo;
			FNewReplicatedActorInfo& ActorInfo = DynamicActorInfo.ActorInfo;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (!IsActorValidForReplicationGather(DynamicActor))
			{
				UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::PrepareForReplication: Dynamic Actor no longer ready for replication");
				UE_LOGF(LogReplicationGraph, Warning, "%ls", *GetNameSafe(DynamicActor));
				continue;
			}
#endif

			// Update location
			FGlobalActorReplicationInfo& ActorRepInfo = GlobalRepMap->Get(DynamicActor);

			// Check if this resets spatial bias
			const FVector Location3D = DynamicActor->GetActorLocation();
			ActorRepInfo.WorldLocation = Location3D;
			
			if (WillActorLocationGrowSpatialBounds(Location3D))
			{
				HandleActorOutOfSpatialBounds(DynamicActor, Location3D, false);
			}

			if (!bNeedsRebuild)
			{
				// Get the new CellInfo
				const FActorCellInfo NewCellInfo = GetCellInfoForActor(DynamicActor, Location3D, ActorRepInfo.Settings.GetCullDistance());

				if (PreviousCellInfo.IsValid())
				{
					bool bDirty = false;

					if (UNLIKELY(NewCellInfo.StartX > PreviousCellInfo.EndX || NewCellInfo.EndX < PreviousCellInfo.StartX ||
							NewCellInfo.StartY > PreviousCellInfo.EndY || NewCellInfo.EndY < PreviousCellInfo.StartY))
					{
						// No longer intersecting, we just have to remove from all previous nodes and add to all new nodes
						
						bDirty = true;

						GetGridNodesForActor(DynamicActor, PreviousCellInfo, GatheredNodes);
						for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
						{
							Node->RemoveDynamicActor(ActorInfo);
						}

						GetGridNodesForActor(DynamicActor, NewCellInfo, GatheredNodes);
						for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
						{
							Node->AddDynamicActor(ActorInfo);
						}
					}
					else
					{
						// Some overlap so lets find out what cells need to be added or removed

						if (PreviousCellInfo.StartX < NewCellInfo.StartX)
						{
							// We lost columns on the left side
							bDirty = true;
						
							for (int32 X = PreviousCellInfo.StartX; X < NewCellInfo.StartX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = PreviousCellInfo.StartY; Y <= PreviousCellInfo.EndY; ++Y)
								{
									if (auto& Node = GetCell(GridX, Y))
									{
										Node->RemoveDynamicActor(ActorInfo);
									}
								}
							}
						}
						else if(PreviousCellInfo.StartX > NewCellInfo.StartX)
						{
							// We added columns on the left side
							bDirty = true;

							for (int32 X = NewCellInfo.StartX; X < PreviousCellInfo.StartX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = NewCellInfo.StartY; Y <= NewCellInfo.EndY; ++Y)
								{
									GetCellNode(GetCell(GridX,Y))->AddDynamicActor(ActorInfo);
								}
							}

						}

						if (PreviousCellInfo.EndX < NewCellInfo.EndX)
						{
							// We added columns on the right side
							bDirty = true;

							for (int32 X = PreviousCellInfo.EndX+1; X <= NewCellInfo.EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = NewCellInfo.StartY; Y <= NewCellInfo.EndY; ++Y)
								{
									GetCellNode(GetCell(GridX,Y))->AddDynamicActor(ActorInfo);
								}
							}
						
						}
						else if(PreviousCellInfo.EndX > NewCellInfo.EndX)
						{
							// We lost columns on the right side
							bDirty = true;

							for (int32 X = NewCellInfo.EndX+1; X <= PreviousCellInfo.EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = PreviousCellInfo.StartY; Y <= PreviousCellInfo.EndY; ++Y)
								{
									if (auto& Node = GetCell(GridX, Y))
									{
										Node->RemoveDynamicActor(ActorInfo);
									}
								}
							}
						}

						// --------------------------------------------------

						// We've handled left/right sides. So while handling top and bottom we only need to worry about this run of X cells
						const int32 StartX = FMath::Max<int32>(NewCellInfo.StartX, PreviousCellInfo.StartX);
						const int32 EndX = FMath::Min<int32>(NewCellInfo.EndX, PreviousCellInfo.EndX);

						if (PreviousCellInfo.StartY < NewCellInfo.StartY)
						{
							// We lost rows on the top side
							bDirty = true;
							
							for (int32 X = StartX; X <= EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = PreviousCellInfo.StartY; Y < NewCellInfo.StartY; ++Y)
								{
									if (auto& Node = GetCell(GridX, Y))
									{
										Node->RemoveDynamicActor(ActorInfo);
									}
								}
							}
						}
						else if(PreviousCellInfo.StartY > NewCellInfo.StartY)
						{
							// We added rows on the top side
							bDirty = true;
							
							for (int32 X = StartX; X <= EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = NewCellInfo.StartY; Y < PreviousCellInfo.StartY; ++Y)
								{
									GetCellNode(GetCell(GridX,Y))->AddDynamicActor(ActorInfo);
								}
							}
						}

						if (PreviousCellInfo.EndY < NewCellInfo.EndY)
						{
							// We added rows on the bottom side
							bDirty = true;
							
							for (int32 X = StartX; X <= EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = PreviousCellInfo.EndY+1; Y <= NewCellInfo.EndY; ++Y)
								{
									GetCellNode(GetCell(GridX,Y))->AddDynamicActor(ActorInfo);
								}
							}
						}
						else if (PreviousCellInfo.EndY > NewCellInfo.EndY)
						{
							// We lost rows on the bottom side
							bDirty = true;
							
							for (int32 X = StartX; X <= EndX; ++X)
							{
								auto& GridX = GetGridX(X);
								for (int32 Y = NewCellInfo.EndY+1; Y <= PreviousCellInfo.EndY; ++Y)
								{
									if (auto& Node = GetCell(GridX, Y))
									{
										Node->RemoveDynamicActor(ActorInfo);
									}
								}
							}
						}
					}

					if (bDirty)
					{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						if (CVar_RepGraph_Spatial_DebugDynamic)
						{
							auto CellInfoStr = [](const FActorCellInfo& CellInfo) { return FString::Printf(TEXT("[%d,%d]-[%d,%d]"), CellInfo.StartX, CellInfo.StartY, CellInfo.EndX, CellInfo.EndY); };
							UE_LOGF(LogReplicationGraph, Display, "%ls moved cells. From %ls to %ls", *GetActorRepListTypeDebugString(DynamicActor), *CellInfoStr(PreviousCellInfo), *CellInfoStr(NewCellInfo));

							const int32 MinX = FMath::Min<int32>(PreviousCellInfo.StartX, NewCellInfo.StartX);
							const int32 MinY = FMath::Min<int32>(PreviousCellInfo.StartY, NewCellInfo.StartY);
							const int32 MaxX = FMath::Max<int32>(PreviousCellInfo.EndX, NewCellInfo.EndX);
							const int32 MaxY = FMath::Max<int32>(PreviousCellInfo.EndY, NewCellInfo.EndY);

							
							for (int32 Y = MinY; Y <= MaxY; ++Y)
							{
								FString Str = FString::Printf(TEXT("[%d]   "), Y);
								for (int32 X = MinX; X <= MaxX; ++X)
								{
									const bool bShouldBeInOld = (X >= PreviousCellInfo.StartX && X <= PreviousCellInfo.EndX) && (Y >= PreviousCellInfo.StartY && Y <= PreviousCellInfo.EndY);
									const bool bShouldBeInNew = (X >= NewCellInfo.StartX && X <= NewCellInfo.EndX) && (Y >= NewCellInfo.StartY && Y <= NewCellInfo.EndY);

									bool bInCell = false;
									if (auto& Node = GetCell(GetGridX(X),Y))
									{
										TArray<FActorRepListType> ActorsInCell;
										Node->GetAllActorsInNode_Debugging(ActorsInCell);
										for (auto ActorInCell : ActorsInCell)
										{
											if (ActorInCell == DynamicActor)
											{
												if (bInCell)
												{
													UE_LOGF(LogReplicationGraph, Warning, "  Actor is in cell multiple times! [%d, %d]", X, Y);
												}
												bInCell = true;
											}
										}
									}

									if (bShouldBeInOld && bShouldBeInNew && bInCell)
									{
										// All good, didn't move
										Str += "* ";
									}
									else if (!bShouldBeInOld && bShouldBeInNew && bInCell)
									{
										// All good, add
										Str += "+ ";
									}
									else if (bShouldBeInOld && !bShouldBeInNew && !bInCell)
									{
										// All good, removed
										Str += "- ";
									}
									else if (!bShouldBeInOld && !bShouldBeInNew && !bInCell)
									{
										// nada
										Str += "  ";
									}
									else
									{
										UE_LOGF(LogReplicationGraph, Warning, "  Bad update! Cell [%d,%d]. ShouldBeInOld: %d. ShouldBeInNew: %d. IsInCell: %d", X, Y, bShouldBeInOld, bShouldBeInNew, bInCell);
										Str += "! ";
									}
								}

								UE_LOGF(LogReplicationGraph, Display, "%ls", *Str);
							}
						}
#endif

						PreviousCellInfo = NewCellInfo;
					}
				}
				else
				{
					// First time - Just add
					GetGridNodesForActor(DynamicActor, NewCellInfo, GatheredNodes);
					for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
					{
						Node->AddDynamicActor(ActorInfo);
					}

					PreviousCellInfo = NewCellInfo;
				}
			}
		}
	}

	// -------------------------------------------
	//	Pending Spatial Actors
	// -------------------------------------------
	for (int32 idx=PendingStaticSpatializedActors.Num()-1; idx>=0; --idx)
	{
		FPendingStaticActors& PendingStaticActor = PendingStaticSpatializedActors[idx];
		if (PendingStaticActor.Actor->IsActorInitialized() == false)
		{
			continue;
		}

		FNewReplicatedActorInfo NewActorInfo(PendingStaticActor.Actor);
		FGlobalActorReplicationInfo& GlobalInfo = GraphGlobals->GlobalActorReplicationInfoMap->Get(PendingStaticActor.Actor);

		AddActorInternal_Static_Implementation(NewActorInfo, GlobalInfo, PendingStaticActor.DormancyDriven);

		PendingStaticSpatializedActors.RemoveAtSwap(idx, EAllowShrinking::No);
	}
	
	// -------------------------------------------
	//	Queued Rebuilds
	// -------------------------------------------
	if (bNeedsRebuild)
	{
		QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_RebuildAll);

		UE_LOGF(LogReplicationGraph, Warning, "Rebuilding spatialization graph for bias %ls", *SpatialBias.ToString());
		
		// Tear down all existing nodes first. This marks them pending kill.
		int32 GridsDestroyed(0);
		for (auto& InnerArray : Grid)
		{
			for (UReplicationGraphNode_GridCell*& N : InnerArray)
			{
				if (N)
				{
					N->TearDown();
					N = nullptr;
					++GridsDestroyed;
				}
			}
		}

		// Force a garbage collection. Without this you may hit OOMs if rebuilding spatialization every frame for some period of time. 
		// (Obviously not ideal to ever be doing this. But you are already hitching, might as well GC to avoid OOM crash).
		if (GridsDestroyed >= CVar_RepGraph_NbDestroyedGridsToTriggerGC)
		{
			GEngine->ForceGarbageCollection(true);
		}

		CleanChildNodes(NodeOrdering::IgnoreOrdering);
		
		for (auto& MapIt : DynamicSpatializedActors)
		{
			FActorRepListType& DynamicActor = MapIt.Key;
			if (ensureMsgf(IsActorValidForReplicationGather(DynamicActor), TEXT("%s not ready for replication."), *GetNameSafe(DynamicActor)))
			{
				FCachedDynamicActorInfo& DynamicActorInfo = MapIt.Value;
				FActorCellInfo& PreviousCellInfo = DynamicActorInfo.CellInfo;
				FNewReplicatedActorInfo& ActorInfo = DynamicActorInfo.ActorInfo;

				const FVector Location3D = DynamicActor->GetActorLocation();
				
				FGlobalActorReplicationInfo& ActorRepInfo = GlobalRepMap->Get(DynamicActor);
				ActorRepInfo.WorldLocation = Location3D;

				const FActorCellInfo NewCellInfo = GetCellInfoForActor(DynamicActor, Location3D, ActorRepInfo.Settings.GetCullDistance());

				GetGridNodesForActor(DynamicActor, NewCellInfo, GatheredNodes);
				for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
				{
					Node->AddDynamicActor(ActorInfo);
				}

				PreviousCellInfo = NewCellInfo;
			}
		}

		for (auto& MapIt : StaticSpatializedActors)
		{
			FActorRepListType& StaticActor = MapIt.Key;
			FCachedStaticActorInfo& StaticActorInfo = MapIt.Value;

			if (ensureMsgf(IsActorValidForReplicationGather(StaticActor), TEXT("%s not ready for replication."), *GetNameSafe(StaticActor)))
			{
				PutStaticActorIntoCell(StaticActorInfo.ActorInfo, GlobalRepMap->Get(StaticActor), StaticActorInfo.bDormancyDriven);
			}
		}

		bNeedsRebuild = false;
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_GridSpatialization2D::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
	using namespace UE::Net::Private;

#if WITH_SERVER_CODE
	TArray<FIntPoint, FReplicationGraphConnectionsAllocator> UniqueCurrentGridCells;

	// Consider all users that are in cells for this connection. 
	// From here, generate a list of coordinates, we'll later work through each coordinate pairing
	// to find the cells that are actually active. This reduces redundancy and cache misses.
	for (const FNetViewer& CurViewer : Params.Viewers)
	{
		if (CurViewer.ViewLocation.Z > ConnectionMaxZ)
		{
			continue;
		}

		// Figure out positioning
		FVector ClampedViewLoc = CurViewer.ViewLocation;
		if (GridBounds.IsValid)
		{
			ClampedViewLoc = GridBounds.GetClosestPointTo(ClampedViewLoc);
		}
		else
		{
			// Prevent extreme locations from causing the Grid to grow too large
			ClampedViewLoc = ClampedViewLoc.BoundToCube(RepGraphHalfWorldMax);
		}

		// Find out what bucket the view is in
		int32 CellX = UE::LWC::FloatToIntCastChecked<int32>((ClampedViewLoc.X - SpatialBias.X) / CellSize);
		if (CellX < 0)
		{
			UE_LOGF(LogReplicationGraph, Verbose, "Net view location.X %ls is less than the spatial bias %ls for %ls", *ClampedViewLoc.ToString(), *SpatialBias.ToString(), CurViewer.Connection ? *CurViewer.Connection->Describe() : TEXT("NONE"));
			CellX = 0;
		}

		int32 CellY = UE::LWC::FloatToIntCastChecked<int32>((ClampedViewLoc.Y - SpatialBias.Y) / CellSize);
		if (CellY < 0)
		{
			UE_LOGF(LogReplicationGraph, Verbose, "Net view location.Y %ls is less than the spatial bias %ls for %ls", *ClampedViewLoc.ToString(), *SpatialBias.ToString(), CurViewer.Connection ? *CurViewer.Connection->Describe() : TEXT("NONE"));
			CellY = 0;
		}

		FIntPoint CurGridCell {CellX, CellY};
		
		// If we have not operated on this cell yet (meaning it's not shared by anyone else), gather for it.
		if (!UniqueCurrentGridCells.Contains(CurGridCell))
		{
			if (UReplicationGraphNode_GridCell* CellNode = GetCell(GetGridX(CellX), CellY))
			{
				CellNode->GatherActorListsForConnection(Params);
			}

			UniqueCurrentGridCells.Add(CurGridCell);

			// Reset visible cell TTL
			if (bDestroyDormantDynamicActors && DestroyDormantDynamicActorsCellTTL > 0)
			{
				TArray<UNetReplicationGraphConnection::FVisibleCellInfo>& VisibleCells = Params.ConnectionManager.GetVisibleCellsForNode(this);
				if (UNetReplicationGraphConnection::FVisibleCellInfo* CellInfo = VisibleCells.FindByKey(CurGridCell))
				{
					CellInfo->Lifetime = DestroyDormantDynamicActorsCellTTL;
				}
				else
				{
					VisibleCells.Add(UNetReplicationGraphConnection::FVisibleCellInfo{CurGridCell, DestroyDormantDynamicActorsCellTTL});
				}
			}
		}
	}

	if (bDestroyDormantDynamicActors && CVar_RepGraph_DormantDynamicActorsDestruction > 0 && Params.bIsSelectedForHeavyComputation)
	{
		QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_DestroyDormantDynamicActors);

		FActorRepListRefView& PrevDormantActorList = Params.ConnectionManager.GetPrevDormantActorListForNode(this);
		TArray<UNetReplicationGraphConnection::FVisibleCellInfo>& VisibleCells = Params.ConnectionManager.GetVisibleCellsForNode(this);

		if (!VisibleCells.IsEmpty())
		{
			RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_CollectDestroyedDormantDynamicActors);

			TArray<UReplicationGraphNode_DormancyNode*, TInlineAllocator<RepGraphDormancyNodesInlineBufferSize>> DormancyNodesCache;
			DormancyNodesCache.SetNum(VisibleCells.Num());

			// Do validation and cache dormancy nodes
			for (int32 Index = 0; Index < VisibleCells.Num(); Index++)
			{
				const FIntPoint GridCell = VisibleCells[Index].Location;

#if DO_ENSURE
				bool bPassesValidation = true;

				bPassesValidation &= ensureMsgf(
					(Grid.Num() > GridCell.X && Grid[GridCell.X].Num() > GridCell.Y),
					TEXT("UReplicationGraphNode_GridSpatialization2D::GatherActorListsForConnection: Previously visible cell (%dx%d) is out of bounds due to grid resize, skipping."),
					GridCell.X, GridCell.Y);
				bPassesValidation &= ensureMsgf(
					VisibleCells[Index].Lifetime >= 0,
					TEXT("UReplicationGraphNode_GridSpatialization2D::GatherActorListsForConnection: Cell (%dx%d) lifetime was negative, that shouldn't happen."),
					GridCell.X, GridCell.Y);

				if (!bPassesValidation)
				{
					DormancyNodesCache.RemoveAtSwap(Index, EAllowShrinking::No);
					VisibleCells.RemoveAtSwap(Index--, EAllowShrinking::No);
					continue;
				}
#endif

				if (UReplicationGraphNode_GridCell* PrevCell = GetCell(GetGridX(GridCell.X), GridCell.Y))
				{
					// don't create a node if missing
					constexpr bool bDontCreateDormancyNode = false;
					DormancyNodesCache[Index] = PrevCell->GetDormancyNode(bDontCreateDormancyNode);
				}
			}

			// Merge Dormant actor lists by producing set C such that
			// C = A && !B, where
			// A - set of actors in cells with Lifetime == 0 (i.e. "dead" cells)
			// B - set of actors in cells with Lifetime > 0 (i.e. "observable" cells)
			
			// Set B - GatheredActors, temp list of "observable dormant actors"
			GatheredActors.Reset(RepGraphDormantActorsListReservedBuffer);
			for (int32 Index = 0; Index < VisibleCells.Num(); Index++)
			{
				if (VisibleCells[Index].Lifetime == 0)
				{
					if (UReplicationGraphNode_DormancyNode* DormancyNode = DormancyNodesCache[Index])
					{
						// Add dormant actors from "no longer observable" cell into PrevDormantActorList,
						// but keep in mind that it could be observable from some other cell, that's why we pass second DormantActorList in
						DormancyNode->ConditionalGatherDormantDynamicActors(PrevDormantActorList, Params, &GatheredActors, true);
					}
					// Remove cell after it's processed
					DormancyNodesCache.RemoveAtSwap(Index, EAllowShrinking::No);
					VisibleCells.RemoveAtSwap(Index--, EAllowShrinking::No);
				}
				else
				{
					if (UReplicationGraphNode_DormancyNode* DormancyNode = DormancyNodesCache[Index])
					{
						// Making sure to remove from the PrevDormantActorList since we don't want things added to the DormantActorList to be destroyed anymore
						DormancyNode->ConditionalGatherDormantDynamicActors(GatheredActors, Params, nullptr, false, &PrevDormantActorList);
					}
				}
			}
		}

		if (PrevDormantActorList.Num() > 0)
		{
			RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_FlushDestroyedDormantDynamicActors);

			int32 NumActorsToRemove = ReplicatedDormantDestructionInfosPerFrame;

			UE_LOGF(LogReplicationGraph, Verbose, "UReplicationGraphNode_GridSpatialization2D::GatherActorListsForConnection: Removing %d Actors (List size: %d)", FMath::Min(NumActorsToRemove, PrevDormantActorList.Num()), PrevDormantActorList.Num());

			FGlobalActorReplicationInfoMap* GlobalRepMap = GraphGlobals.IsValid() ? GraphGlobals->GlobalActorReplicationInfoMap : nullptr;

			// any previous dormant actors not in the current node dormant list
			for (int32 i = 0; i < PrevDormantActorList.Num() && NumActorsToRemove > 0; i++)
			{
				FActorRepListType& Actor = PrevDormantActorList[i];

				const FGlobalActorReplicationInfo* GlobalActorInfo = GlobalRepMap != nullptr ? GlobalRepMap->Find(Actor) : nullptr;

				if (FConnectionReplicationActorInfo* ActorInfo = Params.ConnectionManager.ActorInfoMap.Find(Actor))
				{
					if (ActorInfo->bDormantOnConnection)
					{
						// If this actor is NOT set to always relevant, then flag this actor to be dormancy destroyed on this connection's client.
						if (ActorInfo->GetCullDistanceSquared() > 0.0f)
						{
						Params.ConnectionManager.NotifyAddDormantDestructionInfo(Actor);
						}

						ActorInfo->bDormantOnConnection = false;
						// Ideally, no actor info outside this list should be set to true, so we don't have to worry about resetting them.
						// However we could consider iterating through the actor map to reset all of them.
						ActorInfo->bGridSpatilization_AlreadyDormant = false;


						// add back to connection specific dormancy nodes
						// Try to make sure that we're using the stored actor location otherwise we'll end up adding them to nodes they weren't in before
						const FVector& ActorLocation = GlobalActorInfo != nullptr ? GlobalActorInfo->WorldLocation : Actor->GetActorLocation();
						const FActorCellInfo CellInfo = GetCellInfoForActor(Actor, ActorLocation, ActorInfo->GetCullDistance());
						GetGridNodesForActor(Actor, CellInfo, GatheredNodes);

						for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
						{
							if (UReplicationGraphNode_DormancyNode* DormancyNode = Node->GetDormancyNode())
							{
								// Only notify the connection node if this client was previously inside the cell.
								if (UReplicationGraphNode_ConnectionDormancyNode* ConnectionDormancyNode = DormancyNode->GetExistingConnectionNode(Params))
								{
									ConnectionDormancyNode->NotifyActorDormancyFlush(Actor);
								}
							}
						}

						NumActorsToRemove--;
						PrevDormantActorList.RemoveAtSwap(i--);
					}
					else if (ActorInfo->Channel == nullptr)
					{
						//Channel was closed before becoming dormant.  Remove from list
						UE_CLOGF(CVar_RepGraph_Verify, LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::GatherActorListsForConnection: Actor with null channel pointer in Connection's PrevDormantActorList, it is preferred that we remove actors from the PrevDormantActorList when we clear their channel.");
						ActorInfo->bGridSpatilization_AlreadyDormant = false;
						PrevDormantActorList.RemoveAtSwap(i--);
					}
				}
			}
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_GridSpatialization2D::NotifyActorCullDistChange(AActor* Actor, FGlobalActorReplicationInfo& GlobalInfo, float OldDist)
{
	RG_QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_GridSpatialization2D_NotifyActorCullDistChange);

	// If this actor is statically spatialized then we need to remove it and readd it (this is a little wasteful but in practice not common/only happens at startup)
	if (FCachedStaticActorInfo* StaticActorInfo = StaticSpatializedActors.Find(Actor))
	{
		// Remove with old distance
		GetGridNodesForActor(Actor, GetCellInfoForActor(Actor, GlobalInfo.WorldLocation, OldDist), GatheredNodes);
		for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
		{
			Node->RemoveStaticActor(StaticActorInfo->ActorInfo, GlobalInfo, GlobalInfo.bWantsToBeDormant);
		}

		// Add new distances (there is some waste here but this hopefully doesn't happen much at runtime!)
		GetGridNodesForActor(Actor, GetCellInfoForActor(Actor, GlobalInfo.WorldLocation, GlobalInfo.Settings.GetCullDistance()), GatheredNodes);
		for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
		{
			Node->AddStaticActor(StaticActorInfo->ActorInfo, GlobalInfo, StaticActorInfo->bDormancyDriven);
		}
	}
	else if (FCachedDynamicActorInfo* DynamicActorInfo = DynamicSpatializedActors.Find(Actor))
	{
		// Pull dynamic actor out of the grid. We will put it back on the next gather
		
		FActorCellInfo& PreviousCellInfo = DynamicActorInfo->CellInfo;
		if (PreviousCellInfo.IsValid())
		{
			GetGridNodesForActor(Actor, PreviousCellInfo, GatheredNodes);
			for (UReplicationGraphNode_GridCell* Node : GatheredNodes)
			{
				Node->RemoveDynamicActor(DynamicActorInfo->ActorInfo);
			}
			PreviousCellInfo.Reset();
		}
	}
	else
	{

#if !UE_BUILD_SHIPPING
		// Might be in the pending init list
		if (PendingStaticSpatializedActors.FindByKey(Actor) == nullptr)
		{
			UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_GridSpatialization2D::NotifyActorCullDistChange. %ls Changed Cull Distance (%.2f -> %.2f) but is not in static or dynamic actor lists. %ls", *Actor->GetPathName(), OldDist, GlobalInfo.Settings.GetCullDistance(), *GetPathName() );

			// Search the entire grid. This is slow so only enabled if verify is on.
			if (CVar_RepGraph_Verify)
			{
				bool bFound = false;
				for (auto& InnerArray : Grid)
				{
					for (UReplicationGraphNode_GridCell* CellNode : InnerArray)
					{
						if (CellNode)
						{
							TArray<FActorRepListType> AllActors;
							CellNode->GetAllActorsInNode_Debugging(AllActors);
							if (AllActors.Contains(Actor))
							{
								UE_LOGF(LogReplicationGraph, Warning, "  Its in node %ls", *CellNode->GetPathName());
								bFound = true;
							}
						}
					}
				}
				if (!bFound)
				{
					UE_LOGF(LogReplicationGraph, Warning, "  Not in the grid at all!");
				}
			}
		}
#endif
	}
}

// -------------------------------------------------------

UReplicationGraphNode_AlwaysRelevant::UReplicationGraphNode_AlwaysRelevant()
{
	bRequiresPrepareForReplicationCall = true;
}

void UReplicationGraphNode_AlwaysRelevant::PrepareForReplication()
{
	QUICK_SCOPE_CYCLE_COUNTER(UReplicationGraphNode_AlwaysRelevant_PrepareForReplication);

	if (ChildNode == nullptr)
	{
		ChildNode = CreateChildNode<UReplicationGraphNode_ActorList>();
	}

	ChildNode->NotifyResetAllNetworkActors();
	for (UClass* ActorClass : AlwaysRelevantClasses)
	{
		for (TActorIterator<AActor> It(GetWorld(), ActorClass); It; ++It)
		{
			AActor* Actor = *It;
			if (IsActorValidForReplicationGather(Actor))
			{			
				ChildNode->NotifyAddNetworkActor( FNewReplicatedActorInfo(*It) );
			}
		}
	}
}

void UReplicationGraphNode_AlwaysRelevant::AddAlwaysRelevantClass(UClass* Class)
{
	// Check that we aren't adding sub classes
	for (UClass* ExistingClass : AlwaysRelevantClasses)
	{
		if (ExistingClass->IsChildOf(Class) || Class->IsChildOf(ExistingClass))
		{
			UE_LOGF(LogReplicationGraph, Warning, "UReplicationGraphNode_AlwaysRelevant::AddAlwaysRelevantClass Adding class %ls when %ls is already in the list.", *Class->GetName(), *ExistingClass->GetName());
		}
	}


	AlwaysRelevantClasses.AddUnique(Class);
}

void UReplicationGraphNode_AlwaysRelevant::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
	ChildNode->GatherActorListsForConnection(Params);
}

// -------------------------------------------------------

void UReplicationGraphNode_TearOff_ForConnection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplicationGraphNode_TearOff_ForConnection::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("TearOffActors", TearOffActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReplicationActorList", ReplicationActorList.CountBytes(Ar));
	}
}

void UReplicationGraphNode_TearOff_ForConnection::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
#if WITH_SERVER_CODE

	if (TearOffActors.Num() > 0)
	{
		ReplicationActorList.Reset();
		FPerConnectionActorInfoMap& ActorInfoMap = Params.ConnectionManager.ActorInfoMap;

		for (int32 idx=TearOffActors.Num()-1; idx >=0; --idx)
		{
			FTearOffActorInfo& TearOffInfo = TearOffActors[idx];

			AActor* Actor = TearOffInfo.Actor;
			const uint32 TearOffFrameNum = TearOffInfo.TearOffFrameNum;

			//UE_LOGF(LogReplicationGraph, Display, "UReplicationGraphNode_TearOff_ForConnection::GatherActorListsForConnection. Actor: %ls. GetTearOff: %d. FrameNum: %d. TearOffFrameNum: %d", *GetNameSafe(Actor), (int32)Actor->GetTearOff(), Params.ReplicationFrameNum, TearOffFrameNum);

			// If actor is still valid (not pending kill etc)
			if (Actor && IsActorValidForReplication(Actor))
			{
				// And has not replicated since becoming torn off
				if (FConnectionReplicationActorInfo* ActorInfo = ActorInfoMap.Find(Actor))
				{
					//UE_LOGF(LogReplicationGraph, Display, "0x%X ActorInfo->LastRepFrameNum: %d  ActorInfo->NextReplicationFrameNum: %d. (%d)", (int64)ActorInfo, ActorInfo->LastRepFrameNum, ActorInfo->NextReplicationFrameNum, (ActorInfo->NextReplicationFrameNum - Params.ReplicationFrameNum));

					// Keep adding it to the out list until its replicated at least once. Saturation can prevent it from happening on any given frame.
					// But we could also rep, get an ack for the close, clear the actor's ActorInfo (set LastRepFrameNum = 0), and "miss it". So track that here with bHasReppedOnce
					if (ActorInfo->LastRepFrameNum <= TearOffFrameNum && !(ActorInfo->LastRepFrameNum <= 0 && TearOffInfo.bHasReppedOnce))
					{
						// Add it to the rep list
						ReplicationActorList.Add(Actor);
						TearOffInfo.bHasReppedOnce = true;
						continue;
					}
				}
			}

			//UE_LOGF(LogReplicationGraph, Display, "Removing tearOffActor: %ls. GetTearOff: %d", *GetNameSafe(Actor), (int32)Actor->GetTearOff());

			// If we didn't get added to the list, remove this
			TearOffActors.RemoveAtSwap(idx, EAllowShrinking::No);
		}

		if (ReplicationActorList.Num() > 0)
		{
			Params.OutGatheredReplicationLists.AddReplicationActorList(ReplicationActorList);
		}
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_TearOff_ForConnection::NotifyTearOffActor(AActor* Actor, uint32 FrameNum)
{
	TearOffActors.Emplace( Actor, FrameNum);
}

// -------------------------------------------------------

void UReplicationGraphNode_AlwaysRelevant_ForConnection::AddCachedRelevantActor(const FConnectionGatherActorListParameters& Params, AActor* NewActor, TWeakObjectPtr<AActor>& LastActor)
{
	UpdateCachedRelevantActor(Params, NewActor, LastActor);

	if (NewActor && !ReplicationActorList.Contains(NewActor))
	{
		ReplicationActorList.Add(NewActor);
	}
}

void UReplicationGraphNode_AlwaysRelevant_ForConnection::UpdateCachedRelevantActor(const FConnectionGatherActorListParameters& Params, AActor* NewActor, TWeakObjectPtr<AActor>& LastActor)
{
	if (NewActor != LastActor)
	{
		if (NewActor)
		{
			// Zero out new actor cull distance
			Params.ConnectionManager.ActorInfoMap.FindOrAdd(NewActor).SetCullDistanceSquared(0.f);
		}

		if (AActor* PrevActor = LastActor.Get())
		{
			// Reset previous actor culldistance
			FConnectionReplicationActorInfo& ActorInfo = Params.ConnectionManager.ActorInfoMap.FindOrAdd(PrevActor);
			ActorInfo.SetCullDistanceSquared(GraphGlobals->GlobalActorReplicationInfoMap->Get(PrevActor).Settings.GetCullDistanceSquared());
		}

		LastActor = NewActor;
	}
}

void UReplicationGraphNode_AlwaysRelevant_ForConnection::GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params)
{
	// Call super to add any actors that were explicitly given to use via NotifyAddNetworkActor
	Super::GatherActorListsForConnection(Params);

#if WITH_SERVER_CODE
	// Reset and rebuild another list that will contains our current viewer/viewtarget
	ReplicationActorList.Reset();

	for (const FNetViewer& CurViewer : Params.Viewers)
	{
		if (CurViewer.Connection == nullptr)
		{
			continue;
		}

		// Ignore other connection view targets when this is a replay connection
		if (Params.ConnectionManager.NetConnection->IsReplay() && (CurViewer.Connection != Params.ConnectionManager.NetConnection))
		{
			continue;
		}

		FCachedAlwaysRelevantActorInfo& LastData = PastRelevantActorMap.FindOrAdd(CurViewer.Connection);

		AddCachedRelevantActor(Params, CurViewer.InViewer, LastData.LastViewer);
		AddCachedRelevantActor(Params, CurViewer.ViewTarget, LastData.LastViewTarget);
	}

	CleanupCachedRelevantActors(PastRelevantActorMap);

	if (ReplicationActorList.Num() > 0)
	{
		Params.OutGatheredReplicationLists.AddReplicationActorList(ReplicationActorList);
	}
#endif // WITH_SERVER_CODE
}

void UReplicationGraphNode_AlwaysRelevant_ForConnection::TearDown()
{
	Super::TearDown();

	ReplicationActorList.TearDown();
}

void UReplicationGraphNode_AlwaysRelevant_ForConnection::OnCollectActorRepListStats(FActorRepListStatCollector& StatsCollector) const
{
	StatsCollector.VisitRepList(this, ReplicationActorList);

	Super::OnCollectActorRepListStats(StatsCollector);
}

// -------------------------------------------------------
#if WITH_SERVER_CODE

FAutoConsoleCommandWithWorldAndArgs NetRepGraphPrintChannelCounters(TEXT("Net.RepGraph.PrintActorChannelCounters"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (auto& MapIt: ActorChannelCreateCounter)
		{
			FActorConnectionPair& Pair = MapIt.Key;
			int32 Count = MapIt.Value;
			UE_LOGF(LogReplicationGraph, Display, "%ls : %ls ----> %d", *GetNameSafe(Pair.Actor.Get()), *GetNameSafe(Pair.Connection.Get()), Count );
		}
	})
);

// ------------------------------------------------------------------------------

FAutoConsoleCommandWithWorldAndArgs ChangeActorDiscoveryBudget(TEXT("Net.RepGraph.ActorDiscoveryBudget"), TEXT("Set a separate network traffic budget for data sent when opening a new actor channel. Value in kilobytes per second"), FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& Args, UWorld* World)
{
	int32 BudgetInKBPS = 0;
	if (Args.Num() > 0)
	{
		LexTryParseString<int32>(BudgetInKBPS, *Args[0]);
	}

	for (TObjectIterator<UReplicationGraph> It; It; ++It)
	{
		It->SetActorDiscoveryBudget(BudgetInKBPS);
	}
}));

#endif // WITH_SERVER_CODE
