// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationGraph.h"

#include "Misc/CoreDelegates.h"
#include "Engine/ActorChannel.h"
#include "GameFramework/PlayerController.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Engine.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"
#include "Engine/NetConnection.h"

/**
 *	
 *	===================== Debugging Tools (WIP) =====================
 *	
 *	Net.RepGraph.PrintGraph												Prints replication graph to log (hierarchical representation of graph and its lists)
 *	Net.RepGraph.DrawGraph												Draws replication graph on HUD
 *	
 *	Net.RepGraph.PrintAllActorInfo <MatchString>						Prints global and connection specific info about actors whose pathname contains MatchString. Can be called from client.
 *	
 *	Net.RepGraph.PrioritizedLists.Print	<ConnectionIdx>					Prints prioritized replication list to log 
 *	Net.RepGraph.PrioritizedLists.Draw <ConnectionIdx>					Draws prioritized replication list on HUD
 *	
 *	Net.RepGraph.PrintAll <Frames> <ConnectionIdx> <"Class"/"Num">		Prints the replication graph and prioritized list for given ConnectionIdx for given Frames.
 *	
 *	Net.PacketBudget.HUD												Draws Packet Budget details on HUD
 *	Net.PacketBudget.HUD.Toggle											Toggles capturing/updating the Packet Budget details HUD
 *	
 *	Net.RepGraph.Lists.DisplayDebug										Displays RepActoList stats on HUD
 *	Net.RepGraph.Lists.Stats											Prints RepActorList stats to Log
 *	Net.RepGraph.Lists.Details											Prints extended RepActorList details to log
 *	
 *	Net.RepGraph.StarvedList <ConnectionIdx>							Prints actor starvation stats to HUD
 *	
 *	Net.RepGraph.SetDebugActor <ClassName>								Call on client: sets server debug actor to the closest actor that matches ClassName. See RepGraphConditionalActorBreakpoint
 *	
 */

namespace RepGraphDebugging
{
	UReplicationGraph* FindReplicationGraphHelper(const TArray<FString>& Args)
	{
		FName NetDriverName = NAME_GameNetDriver;

		const FString* DriverStr = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("driver=")); });
		if (DriverStr)
		{
			FParse::Value(**DriverStr, TEXT("driver="), NetDriverName);
		}

		UReplicationGraph* Graph = nullptr;
		for (TObjectIterator<UReplicationGraph> It; It; ++It)
		{
			if (It->NetDriver && (NetDriverName == NAME_None || It->NetDriver->NetDriverName == NetDriverName) && It->NetDriver->GetNetMode() != NM_Client)
			{
				Graph = *It;
				break;
			}
		}
		return Graph;
	}
}

// ----------------------------------------------------------
//	Console Commands
// ----------------------------------------------------------

UNetConnection* AReplicationGraphDebugActor::GetNetConnection() const
{
	if (ConnectionManager)
	{
		return ConnectionManager->NetConnection;
	}

	if (UNetDriver* Driver = GetNetDriver())
	{
		return Driver->ServerConnection;
	}
	
	return nullptr;
}

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerStartDebugging_Implementation()
{
	UE_LOGF(LogReplicationGraph, Display, "ServerStartDebugging");
	ConnectionManager->bEnableDebugging = true;

	UReplicationGraphNode_GridSpatialization2D* GridNode =  nullptr;
	for (UReplicationGraphNode* Node : ReplicationGraph->GlobalGraphNodes)
	{
		GridNode = Cast<UReplicationGraphNode_GridSpatialization2D>(Node);
		if (GridNode)
		{
			break;
	}
	}

	if (GridNode == nullptr)
	{
		return;
	}

	int32 TotalNumCells = 0; // How many cells have been allocated
	int32 TotalLeafNodes = 0; // How many cells have leaf nodes allocated

	TSet<FActorRepListType> UniqueActors;
	int32 TotalElementsInLists = 0;

	TMap<int32, int32> NumStreamLevelsMap;

	int32 MaxY = 0;
	for (TArray<UReplicationGraphNode_GridCell*>& GridY : GridNode->Grid)
	{
		for (UReplicationGraphNode_GridCell* LeafNode : GridY)
		{
			TotalNumCells++;
			if (LeafNode)
			{
				TotalLeafNodes++;

				TArray<FActorRepListType> NodeActors;
				LeafNode->GetAllActorsInNode_Debugging(NodeActors);

				TotalElementsInLists += NodeActors.Num();
				UniqueActors.Append(NodeActors);
				
				NumStreamLevelsMap.FindOrAdd(LeafNode->StreamingLevelCollection.NumLevels())++;
			}
		}
		
		MaxY = FMath::Max<int32>(MaxY, GridY.Num());
	}

	UE_LOGF(LogReplicationGraph, Display, "Grid Dimensions: %d x %d (%d)", GridNode->Grid.Num(), MaxY, GridNode->Grid.Num() * MaxY);
	UE_LOGF(LogReplicationGraph, Display, "Total Num Cells: %d", TotalNumCells);
	UE_LOGF(LogReplicationGraph, Display, "Total Num Leaf Nodes: %d", TotalLeafNodes);
	UE_LOGF(LogReplicationGraph, Display, "Total List Elements: %d", TotalElementsInLists);
	UE_LOGF(LogReplicationGraph, Display, "Total Unique Spatial Actors: %d", UniqueActors.Num());

	UE_LOGF(LogReplicationGraph, Display, "Stream Levels per grid Frequency Report:");
	NumStreamLevelsMap.ValueSort(TGreater<int32>());
	for (auto It : NumStreamLevelsMap)
	{
		UE_LOGF(LogReplicationGraph, Display, "%d Levels --> %d", It.Key, It.Value);
	}
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphDebugActorStart(TEXT("Net.RepGraph.Debug.Start"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerStartDebugging();
		}
	})
);

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerStopDebugging_Implementation()
{
	
}

// -------------------------------------------------------------
void UReplicationGraph::DebugPrintCullDistances(UNetReplicationGraphConnection* SpecificConnection) const
{
	struct FData
	{
		UClass* Class = nullptr;
		float Dist;
		float ConnectionDist;
		int32 Count;
	};

	TArray<FData> DataList;

	for (auto It = GlobalActorReplicationInfoMap.CreateActorMapIterator(); It; ++It)
	{
		AActor* Actor = It.Key();
		const TUniquePtr<FGlobalActorReplicationInfo>& InfoPtr = It.Value();
		if (!InfoPtr || InfoPtr.Get() == nullptr)
		{
			continue;
		}

		const FGlobalActorReplicationInfo& Info = *InfoPtr.Get();

		float ConnectionCullDist = 0.0f;

		// Optionally find this connection's CullDistance
		if (SpecificConnection)
		{
			FPerConnectionActorInfoMap& ConnectionInfo = SpecificConnection->ActorInfoMap;
			if (FConnectionReplicationActorInfo* ConnectionActorInfo = ConnectionInfo.Find(Actor))
			{
				ConnectionCullDist = ConnectionActorInfo->GetCullDistance();
			}
		}

		bool bFound = false;
		for (FData& ExistingData : DataList)
		{
			if (ExistingData.Class == Actor->GetClass() && 
				FMath::IsNearlyEqual(ExistingData.Dist, Info.Settings.GetCullDistance()) &&
				FMath::IsNearlyEqual(ExistingData.ConnectionDist, ConnectionCullDist))
			{
				ExistingData.Count++;
				bFound = true;
				break;
			}
		}

		if (bFound)
		{
			continue;
		}

		FData NewData;
		NewData.Class = Actor->GetClass();
		NewData.Dist = Info.Settings.GetCullDistance();
		NewData.ConnectionDist = ConnectionCullDist;
		NewData.Count = 1;
		DataList.Add(NewData);
	}

	DataList.Sort([](const FData& LHS, const FData& RHS) { return LHS.Dist < RHS.Dist; });

	for (FData& Data : DataList)
	{
		const UClass* NativeParent = Data.Class;
		while (NativeParent && !NativeParent->IsNative())
		{
			NativeParent = NativeParent->GetSuperClass();
		}

		if( SpecificConnection == nullptr )
		{
			UE_LOGF(LogReplicationGraph, Display, "%ls (%ls) [%d] GlobalCullDistance (%.2f)", *GetNameSafe(Data.Class), *GetNameSafe(NativeParent), Data.Count, Data.Dist);
		}
		else
		{
			UE_LOGF(LogReplicationGraph, Display, "%ls (%ls) [%d] GlobalCullDistance (%.2f) ConnectionCullDistance (%.2f)", *GetNameSafe(Data.Class), *GetNameSafe(NativeParent), Data.Count, Data.Dist, Data.ConnectionDist);
		}
	}
}

void AReplicationGraphDebugActor::ServerPrintCullDistances_Implementation()
{
	PrintCullDistances();
}

void AReplicationGraphDebugActor::PrintCullDistances()
{
	ReplicationGraph->DebugPrintCullDistances(ConnectionManager);
}

// -------------------------------------------------------------

FAutoConsoleCommandWithWorldAndArgs NetRepGraphPrintCullDistancesForConnectionCommand(TEXT("Net.RepGraph.PrintCullDistancesForConnection"),TEXT("Print the cull distances via the ReplicationDebugActor to add the connection cull distances."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerPrintCullDistances();
		}
	})
);

FAutoConsoleCommandWithWorldAndArgs NetRepGraphPrintCullDistancesCommand(TEXT("Net.RepGraph.PrintCullDistances"), TEXT("Print the cull distances of RepGraph actors on the server."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (TObjectIterator<UReplicationGraph> It; It; ++It)
		{
			if (It->NetDriver && It->NetDriver->GetNetMode() != NM_Client)
			{
				It->DebugPrintCullDistances();
				break;
			}
		}
	})
);


// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerPrintAllActorInfo_Implementation(const FString& Str)
{
	PrintAllActorInfo(Str);
}

void AReplicationGraphDebugActor::PrintAllActorInfo(FString MatchString)
{
	auto Matches = [&MatchString](UObject* Obj) { return MatchString.IsEmpty() || Obj->GetPathName().Contains(MatchString); };

	GLog->Logf(TEXT("================================================================"));
	GLog->Logf(TEXT("Printing All Actor Info. Replication Frame: %d. MatchString: %s"), ReplicationGraph->GetReplicationGraphFrame(), *MatchString);
	GLog->Logf(TEXT("================================================================"));


	for (auto ClassRepInfoIt = ReplicationGraph->GlobalActorReplicationInfoMap.CreateClassMapIterator(); ClassRepInfoIt; ++ClassRepInfoIt)
	{
		UClass* Class = Cast<UClass>(ClassRepInfoIt.Key().ResolveObjectPtr());
		if (!Class)
		{
			continue;
		}

		const FClassReplicationInfo& ClassInfo = ClassRepInfoIt.Value();

		if (!Matches(Class))
		{
			continue;
		}

		UClass* ParentClass = Class;
		while (ParentClass && !ParentClass->IsNative() && ParentClass->GetSuperClass() && ParentClass->GetSuperClass() != AActor::StaticClass())
		{
			ParentClass = ParentClass->GetSuperClass();
		}

		GLog->Logf(TEXT(""));
		GLog->Logf(TEXT("ClassInfo for %s (Native: %s)"), *GetNameSafe(Class), *GetNameSafe(ParentClass));
		GLog->Logf(TEXT("  %s"), *ClassInfo.BuildDebugStringDelta());
	}

	for (TActorIterator<AActor> ActorIt(GetWorld()); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;
		if ( IsActorValidForReplication(Actor) == false)
		{
			continue;
		}

		if (!Matches(Actor))
		{
			continue;
		}

		if (FGlobalActorReplicationInfo* Info = ReplicationGraph->GlobalActorReplicationInfoMap.Find(Actor))
		{
			GLog->Logf(TEXT(""));
			GLog->Logf(TEXT("GlobalInfo for %s"), *Actor->GetPathName());
			Info->LogDebugString(*GLog);
		}

		if (FConnectionReplicationActorInfo* Info = ConnectionManager->ActorInfoMap.Find(Actor))
		{
			GLog->Logf(TEXT(""));
			GLog->Logf(TEXT("ConnectionInfo for %s"), *Actor->GetPathName());
			Info->LogDebugString(*GLog);
		}
	}

	GLog->Logf(TEXT(""));
	GLog->Logf(TEXT("sizeof(FGlobalActorReplicationInfo): %d"), sizeof(FGlobalActorReplicationInfo));
	GLog->Logf(TEXT("sizeof(FConnectionReplicationActorInfo): %d"), sizeof(FConnectionReplicationActorInfo));
	GLog->Logf(TEXT("Total GlobalActorReplicationInfoMap Num/Size (Unfiltered): %d elements / %d bytes"), ReplicationGraph->GlobalActorReplicationInfoMap.Num(), ReplicationGraph->GlobalActorReplicationInfoMap.Num() * sizeof(FGlobalActorReplicationInfo) );
	GLog->Logf(TEXT("Total PerConnectionActorInfoMap Num/Size (Unfiltered, for this connection only): %d elements / %d bytes"), ConnectionManager->ActorInfoMap.Num(), ConnectionManager->ActorInfoMap.Num() * sizeof(FConnectionReplicationActorInfo) );
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphPrintAllActorInfoCmd(TEXT("Net.RepGraph.PrintAllActorInfo"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		FString MatchString;
		if (Args.Num() > 0)
		{
			MatchString = Args[0];
		}

		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerPrintAllActorInfo(MatchString);
		}
	})
);

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerCellInfo_Implementation()
{
	TArray<FVector, FReplicationGraphConnectionsAllocator> LocationsSent;
	TArray<UNetConnection*, FReplicationGraphConnectionsAllocator> ConnectionsToConsider;

	UNetConnection* PrimaryNetConnection = GetNetConnection();
	ConnectionsToConsider.Add(PrimaryNetConnection);
	for (UChildConnection* Child : PrimaryNetConnection->Children)
	{
		ConnectionsToConsider.Add((UNetConnection*)Child);
	}

	for (UNetConnection* Connection : ConnectionsToConsider)
	{
		FNetViewer Viewer(Connection, 0.f);

		UReplicationGraphNode_GridSpatialization2D* GridNode = nullptr;
		for (UReplicationGraphNode* Node : ReplicationGraph->GlobalGraphNodes)
		{
			GridNode = Cast<UReplicationGraphNode_GridSpatialization2D>(Node);
			if (GridNode)
			{
				break;
			}
		}

		if (GridNode == nullptr)
		{
			return;
		}

		const int32 CellX = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>((Viewer.ViewLocation.X - GridNode->SpatialBias.X) / GridNode->CellSize));
		const int32 CellY = FMath::Max<int32>(0, UE::LWC::FloatToIntCastChecked<int32>((Viewer.ViewLocation.Y - GridNode->SpatialBias.Y) / GridNode->CellSize)); 

		FVector CellLocation(GridNode->SpatialBias.X + (((float)(CellX)+0.5f) * GridNode->CellSize), GridNode->SpatialBias.Y + (((float)(CellY)+0.5f) * GridNode->CellSize), Viewer.ViewLocation.Z);

		if (LocationsSent.Contains(CellLocation))
		{
			UE_LOGF(LogReplicationGraph, Verbose, "Skipping location %ls as we've already sent it", *(CellLocation.ToString()));
			continue;
		}

		LocationsSent.Add(CellLocation);

		TArray<FActorRepListType> ActorsInCell;
		FVector CellExtent(GridNode->CellSize, GridNode->CellSize, 10.f);

		if (GridNode->Grid.IsValidIndex(CellX))
		{
			TArray<UReplicationGraphNode_GridCell*>& GridY = GridNode->Grid[CellX];
			if (GridY.IsValidIndex(CellY))
			{
				if (UReplicationGraphNode_GridCell* LeafNode = GridY[CellY])
				{
					LeafNode->GetAllActorsInNode_Debugging(ActorsInCell);
				}
			}
		}

#if !UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY
		ClientCellInfo(CellLocation, CellExtent, ActorsInCell);
#else
		TArray<AActor*> RealActorsInCell;
		for (const FActorRepListType& RepListActor : ActorsInCell)
		{
			RealActorsInCell.Add(RepListActor);
		}
		ClientCellInfo(CellLocation, CellExtent, RealActorsInCell);
#endif
	}	
}

void AReplicationGraphDebugActor::ClientCellInfo_Implementation(FVector CellLocation, FVector CellExtent, const TArray<AActor*>& Actors)
{
	DrawDebugBox(GetWorld(), CellLocation, CellExtent, FColor::Blue, true, 10.f);

	int32 NullActors=0;
	for (const AActor* Actor : Actors)
	{
		if (Actor)
		{
			DrawDebugLine(GetWorld(), CellLocation, Actor->GetActorLocation(), FColor::Blue, true, 10.f);
		}
		else
		{
			NullActors++;
		}
	}

	UE_LOGF(LogReplicationGraph, Display, "NullActors: %d", NullActors);
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphCellInfo(TEXT("Net.RepGraph.Spatial.CellInfo"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerCellInfo();
		}
	})
);

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerSetCullDistanceForClass_Implementation(UClass* Class, float CullDistance)
{
	if (!Class)
	{
		UE_LOGF(LogReplicationGraph, Display, "Invalid Class");
		return;
	}

	const float CullDistSq = CullDistance * CullDistance;

	FClassReplicationInfo& ClassInfo = ReplicationGraph->GlobalActorReplicationInfoMap.GetClassInfo(Class);
	ClassInfo.SetCullDistanceSquared(CullDistSq);
	UE_LOGF(LogReplicationGraph, Display, "Setting cull distance for class %ls to %.2f", *Class->GetName(), CullDistance);

	for (TActorIterator<AActor> ActorIt(GetWorld(), Class); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;
		if (FGlobalActorReplicationInfo* ActorInfo = ReplicationGraph->GlobalActorReplicationInfoMap.Find(Actor))
		{
			ActorInfo->Settings.SetCullDistanceSquared(CullDistSq);
			UE_LOGF(LogReplicationGraph, Display, "Setting GlobalActorInfo cull distance for %ls to %.2f", *Actor->GetName(), CullDistance);
		}


		if (FConnectionReplicationActorInfo* ConnectionActorInfo = ConnectionManager->ActorInfoMap.Find(Actor))
		{
			ConnectionActorInfo->SetCullDistanceSquared(CullDistSq);
			UE_LOGF(LogReplicationGraph, Display, "Setting Connection cull distance for %ls to %.2f", *Actor->GetName(), CullDistance);
		}
	}
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphSetClassCullDistance(TEXT("Net.RepGraph.SetClassCullDistance"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() <= 1)
		{
			UE_LOGF(LogReplicationGraph, Display, "Usage: Net.RepGraph.SetClassCullDistance <Class> <Distance>");
			return;
		}

		UClass* Class = UClass::TryFindTypeSlow<UClass>(Args[0]);
		if (Class == nullptr)
		{
			UE_LOGF(LogReplicationGraph, Display, "Could not find Class: %ls", *Args[0]);
			return;
		}

		float Distance = 0.f;
		if (!LexTryParseString<float>(Distance, *Args[1]))
		{
			UE_LOGF(LogReplicationGraph, Display, "Could not parse %ls as float.", *Args[1]);
		}

		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerSetCullDistanceForClass(Class, Distance);
		}
	})
);

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerSetPeriodFrameForClass_Implementation(UClass* Class, int32 PeriodFrame)
{
	if (!Class)
	{
		UE_LOGF(LogReplicationGraph, Display, "Invalid Class");
		return;
	}

	if (PeriodFrame < 0 || PeriodFrame > MAX_uint16)
	{
		UE_LOGF(LogReplicationGraph, Display, "Invalid PeriodFrame");
		return;
	}

	FClassReplicationInfo& ClassInfo = ReplicationGraph->GlobalActorReplicationInfoMap.GetClassInfo(Class);
	ClassInfo.ReplicationPeriodFrame = static_cast<uint16>(PeriodFrame);
	UE_LOGF(LogReplicationGraph, Display, "Setting ReplicationPeriodFrame for class %ls to %u", *Class->GetName(), ClassInfo.ReplicationPeriodFrame);

	for (TActorIterator<AActor> ActorIt(GetWorld(), Class); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;
		if (FGlobalActorReplicationInfo* ActorInfo = ReplicationGraph->GlobalActorReplicationInfoMap.Find(Actor))
		{
			ActorInfo->Settings.ReplicationPeriodFrame = static_cast<uint16>(PeriodFrame);
			UE_LOGF(LogReplicationGraph, Display, "Setting GlobalActorInfo ReplicationPeriodFrame for %ls to %u", *Actor->GetName(), ActorInfo->Settings.ReplicationPeriodFrame);
		}


		if (FConnectionReplicationActorInfo* ConnectionActorInfo = ConnectionManager->ActorInfoMap.Find(Actor))
		{
			ConnectionActorInfo->ReplicationPeriodFrame = static_cast<uint16>(PeriodFrame);
			UE_LOGF(LogReplicationGraph, Display, "Setting Connection ReplicationPeriodFrame for %ls to %u", *Actor->GetName(), ConnectionActorInfo->ReplicationPeriodFrame);
		}
	}
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphSetPeriodFrame(TEXT("Net.RepGraph.SetPeriodFrame"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() <= 1)
		{
			UE_LOGF(LogReplicationGraph, Display, "Usage: Net.RepGraph.SetPeriodFrame <Class> <PeriodFrameNum>");
			return;
		}

		UClass* Class = UClass::TryFindTypeSlow<UClass>(Args[0]);
		if (Class == nullptr)
		{
			UE_LOGF(LogReplicationGraph, Display, "Could not find Class: %ls", *Args[0]);
			return;
		}

		float Distance = 0.f;
		if (!LexTryParseString<float>(Distance, *Args[1]))
		{
			UE_LOGF(LogReplicationGraph, Display, "Could not parse %ls as float.", *Args[1]);
		}

		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerSetPeriodFrameForClass(Class, UE::LWC::FloatToIntCastChecked<int32>(Distance));
		}
	})
);

// -------------------------------------------------------------

void AReplicationGraphDebugActor::ServerSetConditionalActorBreakpoint_Implementation(AActor* Actor)
{
	DebugActorConnectionPair.Actor = Actor;
	DebugActorConnectionPair.Connection = Actor ? this->GetNetConnection() : nullptr; // clear connection if null actor was sent

	UE_LOGF(LogReplicationGraph, Display, "AReplicationGraphDebugActor::ServerSetConditionalActorBreakpoint set to %ls/%ls", *GetPathNameSafe(Actor), DebugActorConnectionPair.Connection.IsValid() ? *DebugActorConnectionPair.Connection->Describe() : TEXT("Null") );
}

FAutoConsoleCommandWithWorldAndArgs NetRepGraphSetDebugActorConnectionCmd(TEXT("Net.RepGraph.SetDebugActor"),TEXT("Set DebugActorConnectionPair on server, from client. Specify  "),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		UE_LOGF(LogReplicationGraph, Display, "Usage: Net.RepGraph.SetDebugActor <Class>");

		APlayerController* PC = GEngine->GetFirstLocalPlayerController(World);
		if (!PC)
		{
			UE_LOGF(LogReplicationGraph, Display, "No PC found!");
			return;
		}

		AActor* NewDebugActor = nullptr;

		if (Args.Num() <= 0)
		{
			UE_LOGF(LogReplicationGraph, Display, "No class specified. Clearing debug actor!");
		}
		else
		{
			FVector::FReal ClosestMatchDistSq = WORLD_MAX;
			AActor* ClosestMatchActor = nullptr;
			FVector CamLoc;
			FRotator CamRot;
			PC->GetPlayerViewPoint(CamLoc, CamRot);

			for (TActorIterator<AActor> ActorIt(World); ActorIt; ++ActorIt)
			{
				AActor* Actor = *ActorIt;
				UClass* Class = Actor->GetClass();

				if (Actor->GetIsReplicated() == false)
				{
					continue;
				}

				while(Class)
				{
					if (Class->GetName().Contains(Args[0]))
					{
						break;
					}
					Class = Class->GetSuperClass();
				}

				if (Class)
				{
					FVector::FReal DistSq = (Actor->GetActorLocation() - CamLoc).SizeSquared2D();
					if (DistSq < ClosestMatchDistSq)
					{
						ClosestMatchDistSq = DistSq;
						ClosestMatchActor = Actor;
					}
				}
			}

			if (ClosestMatchActor)
			{
				UE_LOGF(LogReplicationGraph, Display, "Best Match = %ls. (Class=%ls)", *ClosestMatchActor->GetPathName(), *ClosestMatchActor->GetClass()->GetName());
				NewDebugActor = ClosestMatchActor;
			}
			else
			{
				UE_LOGF(LogReplicationGraph, Display, "Unable to find actor that matched class %ls", *Args[0]);
			}
		}

		for (TActorIterator<AReplicationGraphDebugActor> It(World); It; ++It)
		{
			It->ServerSetConditionalActorBreakpoint(NewDebugActor);
		}
	})
);


// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

#if !(UE_BUILD_SHIPPING)
FAutoConsoleCommandWithWorldAndArgs NetRepGraphSetCellSize(TEXT("Net.RepGraph.Spatial.SetCellSize"), TEXT(""),
FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
{
	float NewGridSize = 0.f;
	if (Args.Num() > 0)
	{
		LexFromString(NewGridSize, *Args[0]);
	}

	if (NewGridSize <= 0.f)
	{
		return;
	}

	for (TObjectIterator<UReplicationGraphNode_GridSpatialization2D> It; It; ++It)
	{
		UReplicationGraphNode_GridSpatialization2D* Node = *It;
		if (Node && Node->HasAnyFlags(RF_ClassDefaultObject) == false)
		{
			Node->CellSize = NewGridSize;
			Node->ForceRebuild();
		}
	}
}));
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
FAutoConsoleCommandWithWorldAndArgs NetRepGraphForceRebuild(TEXT("Net.RepGraph.Spatial.ForceRebuild"),TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		for (TObjectIterator<UReplicationGraphNode_GridSpatialization2D> It; It; ++It)
		{
			UReplicationGraphNode_GridSpatialization2D* Node = *It;
			if (Node && Node->HasAnyFlags(RF_ClassDefaultObject) == false)
			{
				Node->ForceRebuild();
				Node->DebugActorNames.Append(Args);
			}
		}
	})
);

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------



FAutoConsoleCommand RepDriverListsAddTestmd(TEXT("Net.RepGraph.Lists.AddTest"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{ 
	static FActorRepListRefView List;

	int32 Num = 1;
	if (Args.Num() > 0 )
	{
		LexFromString(Num,*Args[0]);
	}
	
	while(Num-- > 0)
	{
		List.Add(nullptr);
	}
}));


FAutoConsoleCommand RepDriverListDetailsCmd(TEXT("Net.RepGraph.Lists.Details"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{ 
	int32 PoolIdx = 0;
	int32 BlockIdx = 0;
	int32 ListIdx = -1;
	
	if (Args.Num() > 0 )
	{
		LexFromString(PoolIdx,*Args[0]);
	}

	if (Args.Num() > 1 )
	{
		LexFromString(BlockIdx,*Args[1]);
	}

	if (Args.Num() > 2 )
	{
		LexFromString(ListIdx,*Args[2]);
	}

	PrintRepListDetails(PoolIdx, BlockIdx, ListIdx);
}));

FAutoConsoleCommand RepDriverListsDisplayDebugCmd(TEXT("Net.RepGraph.Lists.DisplayDebug"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{ 
	static FDelegateHandle Handle;
	static int32 Mode = 0;
	if (Args.Num() > 0 )
	{
		LexFromString(Mode,*Args[0]);
	}

	if (Handle.IsValid())
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(Handle);
		Handle.Reset();
	}
	else
	{
		Handle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
		{
			FStringOutputDevice Str;
			Str.SetAutoEmitLineTerminator(true);
			PrintRepListStatsAr(Mode, Str);

			TArray<FString> Lines;
			Str.ParseIntoArrayLines(Lines, true);

			for (int32 idx=Lines.Num()-1; idx>=0; --idx)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Lines[idx]));
			}
		});
	}
}));

#endif

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

FAutoConsoleCommand RepDriverStarvListCmd(TEXT("Net.RepGraph.StarvedList"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{ 
	static FDelegateHandle Handle;
	static int32 ConnectionIdx = 0;
	if (Args.Num() > 0 )
	{
		LexFromString(ConnectionIdx, *Args[0]);
	}
	if (Handle.IsValid())
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(Handle);
		Handle.Reset();
	}
	else
	{
		Handle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
		{
			UNetDriver* BestNetDriver = nullptr;
			for (TObjectIterator<UNetDriver> It; It; ++It)
			{
				if (It->NetDriverName == NAME_GameNetDriver)
				{
					if (It->ClientConnections.Num() > 0)
					{
						if (UReplicationGraph* RepGraph = Cast<UReplicationGraph>(It->GetReplicationDriver()))
						{
							UNetConnection* Connection = It->ClientConnections[ FMath::Min(ConnectionIdx, It->ClientConnections.Num()-1) ];
						
							for (TObjectIterator<UNetReplicationGraphConnection> ConIt; ConIt; ++ConIt)
							{
								if (ConIt->NetConnection == Connection)
								{
									struct FStarveStruct
									{
										FStarveStruct(AActor* InActor, uint32 InStarvedCount) : Actor(InActor), StarveCount(InStarvedCount) { }
										AActor* Actor = nullptr;
										uint32 StarveCount = 0;
										bool operator<(const FStarveStruct& Other) const { return StarveCount < Other.StarveCount; }
									};
								
									TArray<FStarveStruct> TheList;

									for (auto MapIt = ConIt->ActorInfoMap.CreateIterator(); MapIt; ++MapIt)
									{
										TheList.Emplace(MapIt.Key(), RepGraph->GetReplicationGraphFrame() - MapIt.Value().Get()->LastRepFrameNum);
									}
									TheList.Sort();
								
									for (int32 i=TheList.Num()-1; i >= 0 ; --i)
									{
										OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString( FString::Printf(TEXT("[%d] %s"), TheList[i].StarveCount, *GetNameSafe(TheList[i].Actor)) ) );
									}
								}
							}
						}
					}
				}
			}
			

			//
		});
	}
}));


// --------------------------------------------------------------------------------------------------------------------------------------------
//	Graph Debugging: help log/debug the state of the Replication Graph
// --------------------------------------------------------------------------------------------------------------------------------------------
void LogGraphHelper(FOutputDevice& Ar, const TArray< FString >& Args)
{
	UReplicationGraph* Graph = RepGraphDebugging::FindReplicationGraphHelper(Args);
	if (!Graph)
	{
		return;
	}

	FReplicationGraphDebugInfo DebugInfo(Ar);
	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("nativeclass")) || Str.Contains(TEXT("nclass")) ; }) )
	{
		DebugInfo.Flags = FReplicationGraphDebugInfo::ShowNativeClasses;
	}
	else if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("class")); }) )
	{
		DebugInfo.Flags = FReplicationGraphDebugInfo::ShowClasses;
	}
	else if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("num")); }) )
	{
		DebugInfo.Flags = FReplicationGraphDebugInfo::ShowTotalCount;
	}
	else
	{
		DebugInfo.Flags = FReplicationGraphDebugInfo::ShowActors;
	}

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("empty")); }) )
	{
		DebugInfo.bShowEmptyNodes = true;
	}

	Graph->LogGraph(DebugInfo);
}

FAutoConsoleCommand RepGraphPrintGraph(TEXT("Net.RepGraph.PrintGraph"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{ 
	LogGraphHelper(*GLog, Args);
	
}));

FAutoConsoleCommand RepGraphDrawGraph(TEXT("Net.RepGraph.DrawGraph"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& InArgs) 
{ 
	static FDelegateHandle Handle;
	static TArray< FString > Args;
	Args = InArgs;

	if (Handle.IsValid())
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(Handle);
		Handle.Reset();
	}
	else
	{
		Handle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
		{
			FStringOutputDevice Str;
			Str.SetAutoEmitLineTerminator(true);

			LogGraphHelper(Str, Args);

			TArray<FString> Lines;
			Str.ParseIntoArrayLines(Lines, true);

			for (int32 idx=0; idx < Lines.Num(); ++idx)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Lines[idx]));
			}
		});
	}
}));

// ===========================================================================================================


void FGlobalActorReplicationInfo::LogDebugString(FOutputDevice& Ar) const
{
	Ar.Logf(TEXT("  LastPreReplicationFrame: %d. ForceNetUpdateFrame: %d. WorldLocation: %s. bWantsToBeDormant %d. LastFlushNetDormancyFrame: %d"), LastPreReplicationFrame, ForceNetUpdateFrame, *WorldLocation.ToString(), bWantsToBeDormant, LastFlushNetDormancyFrame);
	Ar.Logf(TEXT("  Settings: %s"), *Settings.BuildDebugStringDelta());

	TArray<AActor*> DependentActors;
	DependentActorList.GetAllActors(DependentActors);

	if (DependentActors.Num() > 0)
	{
		FString DependentActorStr = TEXT("DependentActors: ");
		for (AActor* DependentActor : DependentActors)
		{
			DependentActorStr += GetActorRepListTypeDebugString(DependentActor) + ' ';
		}

		Ar.Logf(TEXT("  %s"), *DependentActorStr);
	}

	if (ParentActorList.Num() > 0)
	{
		FString DependentActorStr = TEXT("IsADependentOf: ");
		for (AActor* ParentDependentActor : ParentActorList)
		{
			DependentActorStr += GetActorRepListTypeDebugString(ParentDependentActor) + ' ';
		}

		Ar.Logf(TEXT("  %s"), *DependentActorStr);
	}
}

void FConnectionReplicationActorInfo::LogDebugString(FOutputDevice& Ar) const
{
	Ar.Logf(TEXT("  Channel: %s"), *(Channel ? Channel->Describe() : TEXT("None")));
	Ar.Logf(TEXT("  CullDistSq: %.2f (%.2f)"), CullDistanceSquared, FMath::Sqrt(CullDistanceSquared));
	Ar.Logf(TEXT("  NextReplicationFrameNum: %d. ReplicationPeriodFrame: %d. LastRepFrameNum: %d. ActorChannelCloseFrameNum: %d. IsDormantOnConnection: %d. TearOff: %d"), NextReplicationFrameNum, ReplicationPeriodFrame, LastRepFrameNum, ActorChannelCloseFrameNum, bDormantOnConnection, bTearOff);
}

void UReplicationGraph::LogGraph(FReplicationGraphDebugInfo& DebugInfo) const
{
	LogGlobalGraphNodes(DebugInfo);
	LogConnectionGraphNodes(DebugInfo);
}

void UReplicationGraph::LogGlobalGraphNodes(FReplicationGraphDebugInfo& DebugInfo) const
{
	for (const UReplicationGraphNode* Node : GlobalGraphNodes)
	{
		Node->LogNode(DebugInfo, Node->GetDebugString());
	}
}

void UReplicationGraph::LogConnectionGraphNodes(FReplicationGraphDebugInfo& DebugInfo) const
{
	for (const UNetReplicationGraphConnection* ConnectionManager: Connections)
	{
		DebugInfo.Log(FString::Printf(TEXT("Connection: %s"), *ConnectionManager->NetConnection->GetPlayerOnlinePlatformName().ToString()));

		DebugInfo.PushIndent();
		for (UReplicationGraphNode* Node : ConnectionManager->ConnectionGraphNodes)
		{
			Node->LogNode(DebugInfo, Node->GetDebugString());
		}
		DebugInfo.PopIndent();
	}
}

void UReplicationGraph::CollectRepListStats(FActorRepListStatCollector& StatCollector) const
{
	for (const UReplicationGraphNode* Node : GlobalGraphNodes)
	{
		Node->DoCollectActorRepListStats(StatCollector);
	}

	for (const UNetReplicationGraphConnection* ConnectionManager : Connections)
	{
		for (UReplicationGraphNode* Node : ConnectionManager->ConnectionGraphNodes)
		{
			Node->DoCollectActorRepListStats(StatCollector);
		}
	}
}

void UReplicationGraph::VerifyActorReferences()
{
	UE_LOGF(LogReplicationGraph, Verbose, "UReplicationGraph::VerifyActorReferences - %ls",
		*GetName());

	VerifyActorReferencesInternal();

	for (const TObjectPtr<UReplicationGraphNode>& Node : GlobalGraphNodes)
	{
		Node->VerifyActorReferences();
	}
}

#if DO_ENABLE_REPGRAPH_DEBUG_ACTOR
AReplicationGraphDebugActor* UReplicationGraph::CreateDebugActor() const
{
	return GetWorld()->SpawnActor<AReplicationGraphDebugActor>();
}
#endif

void UReplicationGraphNode::GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const
{
	for (UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->GetAllActorsInNode_Debugging(OutArray);
		}
	}
}

void UReplicationGraphNode::VerifyActorReferences()
{
	VerifyActorReferencesInternal();

	for (const TObjectPtr<UReplicationGraphNode>& Node : AllChildNodes)
	{
		Node->VerifyActorReferencesInternal();
	}
}

bool UReplicationGraphNode::VerifyActorReference(const FActorRepListType& Actor) const
{
	//This logic should be triggered just before GC, so we can have an invalid actor which still has safely addressable memory
	//However, if we have a case where the Actor was cleaned up or stomped, we would crash on GetNameSafe, so I'm logging the Node separately
	const bool bIsValid = Actor && IsValid(Actor);
	
	UE_CLOGF(!bIsValid, LogReplicationGraph, Error, "VerifyActorReference Invalid Actor in RepGraphNode: %ls",
		*GetName());

	const bool bIsValidLowLevel = Actor && Actor->IsValidLowLevel();

	UE_CLOGF(!bIsValid || !bIsValidLowLevel, LogReplicationGraph, Error,
		"Invalid Actor %ls (%ls) is still referenced in %ls. Actor bIsValid: %d bIsValidLowLevel:%d",
		*GetPathNameSafe(Actor),
		*GetNameSafe(Actor ? Actor->GetClass() : nullptr),
		*GetName(),
		bIsValid,
		bIsValidLowLevel);
		
	return bIsValid;
}

void UReplicationGraphNode::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);

	DebugInfo.PushIndent();
	for (const UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		if (DebugInfo.bShowEmptyNodes == false)
		{
			TArray<FActorRepListType> TempArray;
			ChildNode->GetAllActorsInNode_Debugging(TempArray);
			if (TempArray.Num() == 0)
			{
				continue;
			}
		}

		ChildNode->LogNode(DebugInfo, ChildNode->GetDebugString());
	}
	DebugInfo.PopIndent();
}

void LogActorRepList(FReplicationGraphDebugInfo& DebugInfo, FString Prefix, const FActorRepListRefView& List)
{
	if (List.Num() <= 0)
	{
		return;
	}

	FString ActorListStr = FString::Printf(TEXT("%s [%d Actors] "), *Prefix, List.Num());

	if (DebugInfo.Flags == FReplicationGraphDebugInfo::ShowActors)
	{
		for (FActorRepListType Actor : List)
		{
			ActorListStr += GetActorRepListTypeDebugString(Actor);
			ActorListStr += TEXT(" ");
		}
	}
	else if (DebugInfo.Flags == FReplicationGraphDebugInfo::ShowClasses || DebugInfo.Flags == FReplicationGraphDebugInfo::ShowNativeClasses )
	{
		TMap<UClass*, int32> ClassCount;
		for (FActorRepListType Actor : List)
		{
			UClass* ActorClass = GetActorRepListTypeClass(Actor);
			if (DebugInfo.Flags == FReplicationGraphDebugInfo::ShowNativeClasses)
			{
				while (ActorClass && !ActorClass->HasAllClassFlags(CLASS_Native))
				{
					// We lie: don't show AActor. If its blueprinted from AActor just return the blueprint class.
					if (ActorClass->GetSuperClass() == AActor::StaticClass())
					{
						break;
					}
					ActorClass = ActorClass->GetSuperClass();
				}
			}

			ClassCount.FindOrAdd(ActorClass)++;
		}
		for (auto& MapIt : ClassCount)
		{
			ActorListStr += FString::Printf(TEXT("%s:[%d] "), *GetNameSafe(MapIt.Key), MapIt.Value);
		}
	}
	DebugInfo.Log(ActorListStr);
}

void UReplicationGraphNode_GridSpatialization2D::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);

	DebugInfo.PushIndent();
	
	FString DynamicActorsHeader = FString::Printf(TEXT("Tracking %d dynamic actors:"), DynamicSpatializedActors.Num());
	DebugInfo.Log(DynamicActorsHeader);	
	
	if (DebugInfo.Flags == FReplicationGraphDebugInfo::ShowActors)
	{
		FString ActorList; 
		for (auto& MapIt : DynamicSpatializedActors)
		{
			ActorList += GetActorRepListTypeDebugString(MapIt.Key);
			ActorList += TEXT(" ");
		}

		DebugInfo.Log(ActorList);
	}
	DebugInfo.PopIndent();

	DebugInfo.PushIndent();
	for (const UReplicationGraphNode* ChildNode : AllChildNodes)
	{
		if (DebugInfo.bShowEmptyNodes == false)
		{
			TArray<FActorRepListType> TempArray;
			ChildNode->GetAllActorsInNode_Debugging(TempArray);
			if (TempArray.Num() == 0)
			{
				continue;
			}
		}

		ChildNode->LogNode(DebugInfo, ChildNode->GetDebugString());
	}
	DebugInfo.PopIndent();
}

void UReplicationGraphNode_GridSpatialization2D::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const auto& Pair : DynamicSpatializedActors)
	{
		VerifyActorReference(Pair.Key);
	}

	for (const auto& Pair : StaticSpatializedActors)
	{
		VerifyActorReference(Pair.Key);
	}

	for (const FPendingStaticActors& Item : PendingStaticSpatializedActors)
	{
		VerifyActorReference(Item.Actor);
	}
}

void UReplicationGraphNode_GridCell::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);

	DebugInfo.PushIndent();
	
	DebugInfo.Log(TEXT("Static"));
	DebugInfo.PushIndent();
	LogActorList(DebugInfo);
	DebugInfo.PopIndent();

	if (DynamicNode)
	{
		DynamicNode->LogNode(DebugInfo, TEXT("Dynamic"));
	}
	if (DormancyNode)
	{
		DormancyNode->LogNode(DebugInfo, TEXT("Dormant"));
	}
	DebugInfo.PopIndent();
}

void UReplicationGraphNode_TearOff_ForConnection::LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const
{
	DebugInfo.Log(NodeName);
	DebugInfo.PushIndent();
	LogActorRepList(DebugInfo, TEXT("TearOff"), ReplicationActorList);
	DebugInfo.PopIndent();
}

void UReplicationGraphNode_TearOff_ForConnection::VerifyActorReferencesInternal()
{
	Super::VerifyActorReferencesInternal();

	for (const FActorRepListType& Actor : ReplicationActorList)
	{
		VerifyActorReference(Actor);
	}
}

void UReplicationGraphNode_TearOff_ForConnection::OnCollectActorRepListStats(FActorRepListStatCollector& StatsCollector) const
{
	StatsCollector.VisitRepList(this, ReplicationActorList);
	Super::OnCollectActorRepListStats(StatsCollector);
}

// --------------------------------------------------------------------------------------------------------------------------------------------
//	Prioritization Debugging: help log/debug 
// --------------------------------------------------------------------------------------------------------------------------------------------

void PrintPrioritizedList(FOutputDevice& Ar, UNetReplicationGraphConnection* ConnectionManager, FPrioritizedRepList* PrioritizedList)
{
	UReplicationGraph* RepGraph = ConnectionManager->NetConnection->Driver->GetReplicationDriver<UReplicationGraph>();
	uint32 RepFrameNum = RepGraph->GetReplicationGraphFrame();
	
	// Skipped actors
#if REPGRAPH_DETAILS

	if (PrioritizedList->SkippedDebugDetails.IsValid())
	{
		Ar.Logf(TEXT("[%d Skipped Actors]"), PrioritizedList->SkippedDebugDetails->Num());

		FNativeClassAccumulator DormantClasses;
		FNativeClassAccumulator CulledClasses;

		for (const FSkippedActorFullDebugDetails& SkippedDetails : *PrioritizedList->SkippedDebugDetails)
		{
			FString SkippedStr;
			if (SkippedDetails.bWasDormant)
			{
				SkippedStr = TEXT("Dormant");
				DormantClasses.Increment(SkippedDetails.Actor->GetClass());
			}
			else if (SkippedDetails.DistanceCulled > 0.f)
			{
				SkippedStr = FString::Printf(TEXT("Dist Culled %.2f"), SkippedDetails.DistanceCulled);
				CulledClasses.Increment(SkippedDetails.Actor->GetClass());
			}
			else if (SkippedDetails.FramesTillNextReplication > 0)
			{
				SkippedStr = FString::Printf(TEXT("Not ready (%d frames left)"), SkippedDetails.FramesTillNextReplication);
			}
			else
			{
				SkippedStr = TEXT("Unknown???");
			}

			Ar.Logf(TEXT("%-40s %s"), *GetActorRepListTypeDebugString(SkippedDetails.Actor), *SkippedStr);
		}

		Ar.Logf(TEXT(" Dormant Classes: %s"), *DormantClasses.BuildString());
		Ar.Logf(TEXT(" Culled Classes: %s"), *CulledClasses.BuildString());
	}

#endif

		// Passed (not skipped) actors
	Ar.Logf(TEXT("[%d Passed Actors]"), PrioritizedList->Items.Num());
	for (const FPrioritizedRepList::FItem& Item : PrioritizedList->Items)
	{
		const FConnectionReplicationActorInfo& ActorInfo = ConnectionManager->ActorInfoMap.FindOrAdd(Item.Actor);
		const bool bWasStarved = (ActorInfo.LastRepFrameNum + ActorInfo.ReplicationPeriodFrame) < RepFrameNum;
		FString StarvedString = bWasStarved ? FString::Printf(TEXT(" (Starved %d) "), RepFrameNum - ActorInfo.LastRepFrameNum) : TEXT("");

#if REPGRAPH_DETAILS

		if (PrioritizedList->FullDebugDetails.Get())
		{
			if (FPrioritizedActorFullDebugDetails* FullDetails = PrioritizedList->FullDebugDetails.Get()->FindByKey(Item.Actor))
			{
				Ar.Logf(TEXT("%-40s %.4f %s %s"), *GetActorRepListTypeDebugString(Item.Actor), Item.Priority, *FullDetails->BuildString(), *StarvedString);
				continue;
			}
		}
#endif

		// Simplified version without full details
		UClass* Class = Item.Actor->GetClass();
		while (Class && !Class->IsNative())
		{
			Class = Class->GetSuperClass();
		}

		Ar.Logf(TEXT("%-40s %-20s %.4f %s"), *GetActorRepListTypeDebugString(Item.Actor), *GetNameSafe(Class), Item.Priority, *StarvedString);
	}

	Ar.Logf(TEXT(""));
}

TFunction<void()> LogPrioritizedListHelper(FOutputDevice& Ar, const TArray< FString >& Args, bool bAutoUnregister)
{
	static TWeakObjectPtr<UNetReplicationGraphConnection> WeakConnectionManager;
	static FDelegateHandle Handle;

	static TFunction<void()> ResetFunc = []()
	{
		if (Handle.IsValid() && WeakConnectionManager.IsValid())
		{
			WeakConnectionManager->OnPostReplicatePrioritizeLists.Remove(Handle);
		}
	};

	UReplicationGraph* Graph = RepGraphDebugging::FindReplicationGraphHelper(Args);
	if (!Graph)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Could not find valid Replication Graph.");
		return ResetFunc;
	}

	static int32 ConnectionIdx = 0;
	if (Args.Num() > 0 ) 
	{
		LexFromString(ConnectionIdx, *Args[0]);
	}

	if (Graph->Connections.IsValidIndex(ConnectionIdx) == false)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Invalid ConnectionIdx %d", ConnectionIdx);
		return ResetFunc;
	}

	// Reset if we already have delegates bound
	ResetFunc();
	
	UNetReplicationGraphConnection* ConnectionManager = Graph->Connections[ConnectionIdx];
	WeakConnectionManager = ConnectionManager;

	DO_REPGRAPH_DETAILS(ConnectionManager->bEnableFullActorPrioritizationDetails = true);
	Handle = ConnectionManager->OnPostReplicatePrioritizeLists.AddLambda([&Ar, bAutoUnregister](UNetReplicationGraphConnection* InConnectionManager, FPrioritizedRepList* List)
	{
		PrintPrioritizedList(Ar, InConnectionManager, List);
		if (bAutoUnregister)
		{
			DO_REPGRAPH_DETAILS(InConnectionManager->bEnableFullActorPrioritizationDetails = false);
			InConnectionManager->OnPostReplicatePrioritizeLists.Remove(Handle);
		}
	});

	return ResetFunc;
}

FAutoConsoleCommand RepGraphPrintPrioritizedList(TEXT("Net.RepGraph.PrioritizedLists.Print"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args) 
{
	LogPrioritizedListHelper(*GLog, Args, true);
	
}));

FAutoConsoleCommand RepGraphDrawPrioritizedList(TEXT("Net.RepGraph.PrioritizedLists.Draw"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& InArgs) 
{
	static FDelegateHandle Handle;
	static TArray< FString > Args;
	static FStringOutputDevice Str;
	
	Args = InArgs;
	Str.SetAutoEmitLineTerminator(true);

	const bool bClear = InArgs.ContainsByPredicate([](const FString& InStr) { return InStr.Contains(TEXT("clear")); });

	if (Handle.IsValid())
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(Handle);
		Handle.Reset();
		return;
	}

	if (Handle.IsValid() == false)
	{
		Str.Reset();
		LogPrioritizedListHelper(Str, Args, true);

		Handle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
		{
			TArray<FString> Lines;
			Str.ParseIntoArrayLines(Lines, true);

			for (int32 idx=0; idx < Lines.Num(); ++idx)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Lines[idx]));
			}
		});
	}
}));

// --------------------------------------------------------------------------------------------------------------------------------------------
//	Print/Logging for everything (Replication Graph, Prioritized List, Packet Budget [TODO])
// --------------------------------------------------------------------------------------------------------------------------------------------

FAutoConsoleCommand RepGraphPrintAllCmd(TEXT("Net.RepGraph.PrintAll"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& InArgs) 
{
	static TWeakObjectPtr<UNetReplicationGraphConnection> WeakConnectionManager;
	static TArray< FString > Args;

	Args = InArgs;

	UReplicationGraph* Graph = RepGraphDebugging::FindReplicationGraphHelper(Args);
	if (!Graph)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Could not find valid Replication Graph.");
		return;
	}

	int32 FrameCount = 1;
	if (Args.Num() > 0 )
	{
		LexFromString(FrameCount, *Args[0]);
	}

	int32 ConnectionIdx = 0;
	if (Args.Num() > 1 )
	{
		LexFromString(ConnectionIdx, *Args[1]);
	}

	if (Graph->Connections.IsValidIndex(ConnectionIdx) == false)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Invalid ConnectionIdx %d", ConnectionIdx);
		return;
	}
	UNetReplicationGraphConnection* ConnectionManager = Graph->Connections[ConnectionIdx];

	TSharedPtr<FDelegateHandle> Handle = MakeShareable<FDelegateHandle>(new FDelegateHandle());
	TSharedPtr<int32> FrameCountPtr = MakeShareable<int32>(new int32);
	*FrameCountPtr = FrameCount;

	DO_REPGRAPH_DETAILS(ConnectionManager->bEnableFullActorPrioritizationDetails = true);
	*Handle = ConnectionManager->OnPostReplicatePrioritizeLists.AddLambda([Handle, FrameCountPtr, Graph](UNetReplicationGraphConnection* InConnectionManager, FPrioritizedRepList* List)
	{
		GLog->Logf(TEXT(""));
		GLog->Logf(TEXT("===================================================="));
		GLog->Logf(TEXT("Replication Frame %d"), Graph->GetReplicationGraphFrame());
		GLog->Logf(TEXT("===================================================="));

		LogGraphHelper(*GLog, Args);

		PrintPrioritizedList(*GLog, InConnectionManager, List);
		if (*FrameCountPtr >= 0)
		{
			if (--*FrameCountPtr <= 0)
			{
				DO_REPGRAPH_DETAILS(InConnectionManager->bEnableFullActorPrioritizationDetails = false);
				InConnectionManager->OnPostReplicatePrioritizeLists.Remove(*Handle);
			}
		}
	});
	
}));

FAutoConsoleCommand RepDriverListsStatsCmd(TEXT("Net.RepGraph.Lists.Stats"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	UReplicationGraph* Graph = RepGraphDebugging::FindReplicationGraphHelper(Args);
	if (!Graph)
	{
		UE_LOGF(LogReplicationGraph, Warning, "Could not find valid Replication Graph.");
		return;
	}

	FActorRepListStatCollector StatCollector;
	Graph->CollectRepListStats(StatCollector);

	UE_LOGF(LogReplicationGraph, Display, "Printing ActorRepList stats of %ls", *Graph->GetName());
	StatCollector.PrintCollectedData(*GLog);
}));

void FActorRepListStatCollector::PrintCollectedData(FOutputDevice& Ar)
{
	auto SortByBiggestNumLists = [](const FRepListStats& lhs, const FRepListStats& rhs) -> bool
	{
		return lhs.NumLists >= rhs.NumLists;
	};

	// Print per class stats
	{
		Ar.Logf(TEXT(""));
		Ar.Logf(TEXT("======   Node Class stats ======"));
		PerClassStats.ValueStableSort(SortByBiggestNumLists);

		for (TMap<FName, FRepListStats>::TConstIterator It(PerClassStats); It; ++It)
		{
			FName NodeName = It.Key();
			const FRepListStats& NodeStats = It.Value();

			Ar.Logf(TEXT("%s Lists(%u) Avg Actors per list (%.2f) Biggest List (%d) Avg slack (%.2f) Total bytes (%.3f mb)"),
				*NodeName.ToString(),
				NodeStats.NumLists,
				(float)NodeStats.NumActors / (float)NodeStats.NumLists,
				NodeStats.MaxListSize,
				(float)NodeStats.NumSlack / (float)NodeStats.NumLists,
				(float)NodeStats.NumBytes / (1024.f * 1024.f)
			);
		}

		Ar.Logf(TEXT("====================================="));
	}

	// Print per-streaming level stats
	{
		Ar.Logf(TEXT(""));
		Ar.Logf(TEXT("======   Streaming level stats ======"));
		PerStreamingLevelStats.ValueStableSort(SortByBiggestNumLists);

		for (TMap<FName, FRepListStats>::TConstIterator It(PerStreamingLevelStats); It; ++It)
		{
			FName LevelName = It.Key();
			const FRepListStats& NodeStats = It.Value();

			Ar.Logf(TEXT("%s Lists(%u) Avg Actors per list (%.2f) Biggest List (%d) Avg slack (%.2f) Total bytes (%.3f mb)"),
				*LevelName.ToString(),
				NodeStats.NumLists,
				(float)NodeStats.NumActors / (float)NodeStats.NumLists,
				NodeStats.MaxListSize,
				(float)NodeStats.NumSlack / (float)NodeStats.NumLists,
				(float)NodeStats.NumBytes / (1024.f * 1024.f)
			);
		}

		Ar.Logf(TEXT("====================================="));
	}

}


// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

