// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationGraphModule.h"

IMPLEMENT_MODULE( FReplicationGraphModule, ReplicationGraph );

#include "HAL/IConsoleManager.h"
#include "Misc/HotReloadInterface.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectGlobals.h"

void RecompileReplicationGraph(const TArray<FString>& Args)
{
	GWarn->BeginSlowTask( FText::FromString(TEXT("Recompiling rep graph")), true);
	
	IHotReloadInterface* HotReload = IHotReloadInterface::GetPtr();
	if(HotReload != nullptr)
	{
		TArray< UPackage* > PackagesToRebind;
		UPackage* Package = FindPackage( NULL, TEXT("/Script/ReplicationGraph"));
		if( Package != NULL )
		{
			PackagesToRebind.Add( Package );
		}

		HotReload->RebindPackages(PackagesToRebind, EHotReloadFlags::WaitForCompletion, *GLog);
	}

	GWarn->EndSlowTask();
}

FAutoConsoleCommand RecompileReplicationGraphCmd( TEXT("ReplicationGraph.Reload"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateStatic(&RecompileReplicationGraph) );
