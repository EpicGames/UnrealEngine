// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationGraphTypes.h"

#include "Engine/NetConnection.h"
#include "ReplicationGraph.h"

#include "Misc/ConfigCacheIni.h"
#include "HAL/LowLevelMemStats.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationGraphTypes)

DEFINE_LOG_CATEGORY( LogReplicationGraph );

DECLARE_LLM_MEMORY_STAT(TEXT("NetRepGraph"), STAT_NetRepGraphLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(NetRepGraph, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_NetRepGraphLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

#if WITH_SERVER_CODE && UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY

static FAutoConsoleVariableRef CVar_NetRepGraphUseWeakPointers(
	TEXT("Net.RepGraph.UseWeakPointers"),
	UE::Net::RepGraph::bUseWeakPointers,
	TEXT("Uses weak pointers in RepGraph, validating them on access to detect dangling pointers without crashing."));

#endif

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Actor List Allocator
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/// This could be simplified by using TChunkedArray?

template<uint32 NumListsPerBlock, uint32 MaxNumPools>
struct TActorListAllocator
{
	FActorRepList& RequestList(int32 ExpectedMaxSize)
	{
		return GetOrCreatePoolForListSize(ExpectedMaxSize).RequestList();
	}

	void ReleaseList(FActorRepList& List)
	{
		// Check we are at 0 refs, reset length to 0, and reset the used bit
		checkfSlow(List.RefCount == 0, TEXT("TActorListAllocator::ReleaseList called on list with RefCount %d"), List.RefCount);
		List.UsedBitRef = false;
		List.Num = 0;
	}

	void PreAllocateLists(int32 ListSize, int32 NumLists)
	{
		GetOrCreatePoolForListSize(ListSize, true).PreAllocateLists(NumLists);
	}

	/** Logs stats about this entire allocator. mode = level of detail */
	void LogStats(int32 Mode, FOutputDevice& Ar=*GLog);
	/** Logs details about a specific list */
	void LogDetails(int32 PoolSize, int32 BlockIdx, int32 ListIdx, FOutputDevice& Ar=*GLog);

	void CountBytes(FArchive& Ar) const
	{
		PoolTable.CountBytes(Ar);
		for (const FPool& Pool : PoolTable)
		{
			Pool.CountBytes(Ar);
		}
	}

private:

	/** A pool of lists of the same (max) size. Starts with a single FBlock of NumListsPerBlock lists. More blocks allocated as needed.  */
	struct FPool
	{
		FPool(int32 InListSize) : ListSize(InListSize), Block(InListSize) { }
		
		int32 ListSize;
		bool operator==(const int32& InSize) const { return InSize == ListSize; }

		/** Represents a block of allocated lists. Contains NumListsPerBlock. */
		struct FBlock
		{
			/** Construct a block for given list size. All lists within the block are preallocated and initialized here. */
			FBlock(const int32& InListSize)
			{
				UsedListsBitArray.Init(false, NumListsPerBlock);
				for (int32 i=0; i < NumListsPerBlock; ++i)
				{
					FActorRepList* List = AllocList(InListSize);
					List->RefCount = 0;
					List->Max = InListSize;
					List->Num = 0;
					new (&List->UsedBitRef) FBitReference(UsedListsBitArray[i]); // This is the only way to reassign the bitref (assignment op copies the value from the rhs bitref)
					Lists.Add(List);
				}
			}

			/** Returns next free list on this block of forwards to the next */
			FActorRepList& RequestList(const int32& ReqListSize)
			{
				// Note: we flip the used bit immediately even though the list's refcount==0.
				// FActorRepListView should be the only thing inc/dec refcount. If someone
				// wanted to use TActorListAllocator without ref counting, they could by simply
				// sticking to RequestList/ReleaseList.
				int32 FreeIdx = UsedListsBitArray.FindAndSetFirstZeroBit();
				if (FreeIdx == INDEX_NONE)
				{
					return GetNext(ReqListSize)->RequestList(ReqListSize);
				}

				return Lists[FreeIdx];
			}

			/** Returns next FBlock, allocating it if necessary */
			FBlock* GetNext(const int32& NextListSize)
			{
				if (!Next.IsValid())
				{
					Next = MakeUnique<FBlock>(NextListSize);
				}
				return Next.Get();
			}
			
			void CountBytes(FArchive& Ar) const
			{
				// TIndirectArrays are stored as TArray<void*, Allocator> which means that just calling CountBytes
				// may cause a pretty significant undercount.
				Lists.CountBytes(Ar);
				Ar.CountBytes(sizeof(FActorRepList) * NumListsPerBlock, sizeof(FActorRepList) * NumListsPerBlock);
				for (int32 i = 0; i < NumListsPerBlock; ++i)
				{
					Lists[i].CountBytes(Ar);
				}

				UsedListsBitArray.CountBytes(Ar);

				if (Next.IsValid())
				{
					Ar.CountBytes(sizeof(FBlock), sizeof(FBlock));
					Next->CountBytes(Ar);
				}
			}

			/** Pointers to all the lists we have allocated. This will free allocated FActorRepLists when it is destroyed */
			TIndirectArray<FActorRepList, TFixedAllocator<NumListsPerBlock>>	Lists;

			/** BitArray to track which lists are free. false == free */
			TBitArray<TFixedAllocator<NumListsPerBlock>> UsedListsBitArray;

			/** Pointer to next block, only allocated if necessary */
			TUniquePtr<FBlock> Next;
		};

		/** The head block that we start off with */
		FBlock Block;

		/** Get a free list from this pool */
		FActorRepList& RequestList() { return Block.RequestList(ListSize); }
		
		/** Preallocate (at least) NumLists by allocating (NumLists/NumListsPerBlock) FBlocks */
		void PreAllocateLists(int32 NumLists)
		{
			FBlock* CurrentBlock = &Block;
			while(NumLists > NumListsPerBlock)
			{
				CurrentBlock = CurrentBlock->GetNext(ListSize);
				NumLists -= NumListsPerBlock;
			}
		}

		void CountBytes(FArchive& Ar) const
		{
			Block.CountBytes(Ar);
		}
	};

	/** Fixed size pools. Note that due to inline allocation and the way we store the FBitReference, elements in this container MUST stay stable! A fixed allocator is the best here. It could be a indirect array (slower) or we could make FBlock::Block not inlined (use TUniquePtr)  */
	TArray<FPool, TFixedAllocator<MaxNumPools>>	PoolTable;

	static FActorRepList* AllocList(int32 DataNum)
	{
		uint32 NumBytes = sizeof(FActorRepList) + (DataNum * sizeof(FActorRepListType));
		FActorRepList* NewList = (FActorRepList*)FMemory::Malloc(NumBytes);
		NewList->Max = DataNum;
		NewList->Num = 0;
		NewList->RefCount = 0;
		return NewList;
	}

	FPool& GetOrCreatePoolForListSize(int32 ExpectedMaxSize, bool ForPreAllocation=false)
	{
		FPool* Pool = PoolTable.FindByPredicate([&ExpectedMaxSize](const FPool& InPool) { return ExpectedMaxSize <= InPool.ListSize; });
		if (!Pool)
		{
			QUICK_SCOPE_CYCLE_COUNTER(RepList_Pool_Allocation);
			if (!ForPreAllocation)
			{
				UE_LOGF(LogReplicationGraph, Warning, "No pool big enough for requested list size %d. Creating a new pool. (You may want to preallocate a pool of this size or investigate why this size is needed)", ExpectedMaxSize);
				if (UReplicationGraph::OnListRequestExceedsPooledSize)
				{
					// Rep graph can spew debug info but we don't really know who/why the list is being requested from this point
					UReplicationGraph::OnListRequestExceedsPooledSize(ExpectedMaxSize);
				}
				ExpectedMaxSize = FMath::RoundUpToPowerOfTwo(ExpectedMaxSize); // Round up because this size was not preallocated
			}
			checkf(PoolTable.Num() < MaxNumPools, TEXT("You cannot allocate anymore pools! Consider preallocating a pool of the max list size you will need."));
			Pool = new (PoolTable) FPool( ExpectedMaxSize );
		}
		return *Pool;
	}
};

#ifndef REP_LISTS_PER_BLOCK
#define REP_LISTS_PER_BLOCK 128
#endif
#ifndef REP_LISTS_MAX_NUM_POOLS
#define REP_LISTS_MAX_NUM_POOLS 12
#endif
TActorListAllocator<REP_LISTS_PER_BLOCK, REP_LISTS_MAX_NUM_POOLS> GActorListAllocator;

void FActorRepList::Release()
{
	if (RefCount-- == 1)
	{
		GActorListAllocator.ReleaseList(*this);
	}
}

bool FActorRepListRefView::VerifyContents_Slow() const
{
	for (FActorRepListType Actor : *this)
	{
		if (IsActorValidForReplication(Actor) == false)
		{
			UE_LOGF(LogReplicationGraph, Warning, "Actor %ls not valid for replication", *GetActorRepListTypeDebugString(Actor));
			return false;
		}
	

		TWeakObjectPtr<AActor> WeakPtr(Actor);
		if (WeakPtr.Get() == nullptr)
		{
			UE_LOGF(LogReplicationGraph, Warning, "Actor %ls failed WeakObjectPtr resolve", *GetActorRepListTypeDebugString(Actor));
			return false;
		}
	}

	return true;
}

FString FActorRepListRefView::BuildDebugString() const
{
	FString Str;
	if (Num() > 0)
	{
		Str += GetActorRepListTypeDebugString(RepList[0]);
		for (int32 i = 1; i < Num(); ++i)
		{
			Str += TEXT(", ") + GetActorRepListTypeDebugString(RepList[i]);
		}
	}
	return Str;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void PrintRepListStats(int32 mode)
{
	GActorListAllocator.LogStats(mode);
}
void PrintRepListStatsAr(int32 mode, FOutputDevice& Ar)
{
	GActorListAllocator.LogStats(mode, Ar);
}
void PrintRepListDetails(int32 PoolSize, int32 BlockIdx, int32 ListIdx)
{
	GActorListAllocator.LogDetails(PoolSize, BlockIdx, ListIdx);
}
#endif

void PreAllocateRepList(int32 ListSize, int32 NumLists)
{
	// nothing
}

void CountReplicationGraphSharedBytes_Private(FArchive& Ar)
{
	GActorListAllocator.CountBytes(Ar);
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Stats, Logging, Debugging
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------


#if WITH_EDITOR
void ForEachClientPIEWorld(TFunction<void(UWorld*)> Func)
{
	for (TObjectIterator<UWorld> It; It; ++It)
	{
		if (It->WorldType == EWorldType::PIE && It->GetNetMode() != NM_DedicatedServer)
		{
			Func(*It);
		}
	}
}
#endif

FReplicationGraphCSVTracker::FReplicationGraphCSVTracker()
	: EverythingElse(TEXT("Other"))
	, EverythingElse_FastPath(TEXT("OtherFastPath"))
	, ActorDiscovery(TEXT("ActorDiscovery"))
{
	ResetTrackedClasses();

	GConfig->GetBool(TEXT("ReplicationGraphCSVTracker"), TEXT("bReportUntrackedClasses"), bReportUntrackedClasses, GEngineIni);
}


void LogListDetails(FActorRepList& RepList, FOutputDevice& Ar)
{
	FString ListContentString;
	for ( int32 i=0; i < RepList.Num; ++i)
	{
		ListContentString += GetActorRepListTypeDebugString(RepList.Data[i]);
		if (i < RepList.Num - 1) ListContentString += TEXT(" ");
	}
	Ar.Logf(TEXT("Num: %d. Ref: %d [%s]"), RepList.Num, RepList.RefCount, *ListContentString);
	Ar.Logf(TEXT(""));
}

template<uint32 NumListsPerBlock, uint32 MaxNumPools>
void TActorListAllocator<NumListsPerBlock, MaxNumPools>::LogStats(int32 Mode, FOutputDevice& Ar)
{
	uint32 NumPools = PoolTable.Num();
	uint32 NumBlocks = 0;
	uint32 NumUsedLists = 0;
	uint32 NumElements = 0;
	uint32 NumListBytes = 0;
	for (int32 PoolIdx=0; PoolIdx < PoolTable.Num(); ++PoolIdx)
	{
		FPool& Pool = PoolTable[PoolIdx];

		typename FPool::FBlock* B = &Pool.Block;
		uint32 NumBlocksThisPool = 0;
		uint32 NumUsedThisPool = 0;

		FString BlockBinaryStr;

		while(B)
		{
			NumBlocksThisPool++;
			NumElements += (NumListsPerBlock * Pool.ListSize);
			NumListBytes += NumListsPerBlock * (sizeof(FActorRepList) + (Pool.ListSize * sizeof(FActorRepListType)));

			// Block Details
			if (Mode >= 2)
			{
				for (int32 i=0; i < B->UsedListsBitArray.Num(); ++i)
				{
					BlockBinaryStr += B->UsedListsBitArray[i] ? *LexToString(B->Lists[i].RefCount) : TEXT("0");
				}
				BlockBinaryStr += TEXT(" ");
			}

			uint32 NumUsedThisBlock = 0;
			for (TConstSetBitIterator<TFixedAllocator<NumListsPerBlock>> It(B->UsedListsBitArray); It; ++It)
			{
				NumUsedThisBlock++;

				// List Details
				if (Mode >= 3)
				{
					LogListDetails(B->Lists[It.GetIndex()], Ar);
				}
			}
				
			NumUsedLists += NumUsedThisBlock;
			NumUsedThisPool += NumUsedThisBlock;				

			B = B->Next.Get();
		}

		if (Mode > 1)
		{
			Ar.Logf(TEXT("%s"), *BlockBinaryStr);
		}

		// Pool Details
		if (Mode >= 1)
		{
			Ar.Logf(TEXT("Pool[%d] ListSize: %d. NumBlocks: %d NumUsedLists: %d"), PoolIdx, Pool.ListSize, NumBlocksThisPool, NumUsedThisPool);
		}
		NumUsedLists += NumUsedThisPool;
		NumBlocks += NumBlocksThisPool;
	}

	// All Details
	Ar.Logf(TEXT(""));
	Ar.Logf(TEXT("[TOTAL] NumPools: %u. NumBlocks: %u. NumUsedLists: %u NumElements: %u ListBytes: %u"), NumPools, NumBlocks, NumUsedLists, NumElements, NumListBytes);
}

template<uint32 NumListsPerBlock, uint32 MaxNumPools>
void TActorListAllocator<NumListsPerBlock, MaxNumPools>::LogDetails(int32 PoolSize, int32 BlockIdx, int32 ListIdx, FOutputDevice& Ar)
{
	FPool* Pool = PoolTable.FindByPredicate([&PoolSize](const FPool& InPool) { return PoolSize <= InPool.ListSize; });
	if (!Pool)
	{
		Ar.Logf(TEXT("Could not find suitable PoolSize %d"), PoolSize);
		return;
	}

	if (ListIdx > NumListsPerBlock)
	{
		Ar.Logf(TEXT("ListIdx %d too big. Should be <= %d."), ListIdx, NumListsPerBlock);
		return;
	}

	typename FPool::FBlock* B = &Pool->Block;
	while(B && ListIdx-- > 0)
	{
		B = B->Next.Get();
	}

	if (B)
	{
		if (ListIdx < 0)
		{
			for (TConstSetBitIterator<TFixedAllocator<NumListsPerBlock>> It(B->UsedListsBitArray); It; ++It)
			{
				LogListDetails(B->Lists[It.GetIndex()], Ar);
			}
		}
		else
		{
			LogListDetails(B->Lists[ListIdx], Ar);
		}
	}
}

int32 FGlobalActorReplicationInfoMap::Remove(const FActorRepListType& RemovedActor)
{
	// Clean the references to the removed actor from its dependent link
	if (FGlobalActorReplicationInfo* RemovedActorInfo = Find(RemovedActor))
	{
		RemoveAllActorDependencies(RemovedActor, RemovedActorInfo);
	}

	return ActorMap.Remove(RemovedActor);
}

void FGlobalActorReplicationInfoMap::NotifyActorRenamed(AActor* Actor, FName PreviousStreamingLevelName)
{
	if (FGlobalActorReplicationInfo* RenamedActorInfo = Find(Actor))
	{
		// Update the dependent lists of this renamed actor's parents
		for (AActor* ParentActor : RenamedActorInfo->ParentActorList)
		{
			if (FGlobalActorReplicationInfo* ParentActorInfo = Find(ParentActor))
			{
				ParentActorInfo->DependentActorList.UpdateActorLevel(Actor, PreviousStreamingLevelName);
			}
		}
	}
}

void FGlobalActorReplicationInfoMap::AddDependentActor(AActor* Parent, AActor* Child, FGlobalActorReplicationInfoMap::EWarnFlag WarnFlag)
{
	/**
	* We assume the relationship between the parent and the dependent to be one way: if the parent wants to replicate, then the dependent replicates.
	* Dependent actors can still be routed into another node and replicated independently of their parent if desired, but if the dependent actor
	* isn't being routed anywhere else, its parent must be routed somewhere in the RepGraph in order to ensure the dependent is replicated.
	*/

	const bool bIsParentValid = ensureMsgf(Parent && IsActorValidForReplication(Parent), TEXT("FGlobalActorReplicationInfoMap::AddDependentActor Invalid Parent! %s"),
										   *GetPathNameSafe(Parent));

	const bool bIsChildValid = ensureMsgf(Child && IsActorValidForReplication(Child), TEXT("FGlobalActorReplicationInfoMap::AddDependentActor Invalid Child! %s"),
										  *GetPathNameSafe(Child));

	if (bIsParentValid && bIsChildValid)
	{
		const bool bDoAlreadyDependantWarning = EnumHasAnyFlags(WarnFlag, EWarnFlag::WarnAlreadyDependant);
		const bool bDoNotRegisteredWarning = EnumHasAnyFlags(WarnFlag, EWarnFlag::WarnParentNotRegistered);

		bool bChildIsAlreadyDependant(false);
		if (FGlobalActorReplicationInfo* ParentInfo = Find(Parent))
		{
			bChildIsAlreadyDependant = ParentInfo->DependentActorList.Contains(Child);
			if (bChildIsAlreadyDependant == false)
			{
				if (IsActorValidForReplicationGather(Child))
				{
					ParentInfo->DependentActorList.AddNetworkActor(Child);
				}
			}
		}
		else if (bDoNotRegisteredWarning)
		{
			UE_LOGF(LogReplicationGraph, Warning, "FGlobalActorReplicationInfoMap::AddDependentActor Parent %ls not registered yet, cannot add child %ls", *GetNameSafe(Parent), *GetNameSafe(Child));
		}

		bool bChildHadParentAlready(false);
		if (FGlobalActorReplicationInfo* ChildInfo = Find(Child))
		{
			bChildHadParentAlready = ChildInfo->ParentActorList.Find(Parent) != INDEX_NONE;
			if (bChildHadParentAlready == false)
			{
				if (IsActorValidForReplicationGather(Parent))
				{ 
					ChildInfo->ParentActorList.Add(Parent);
				}
			}
		}

		if (bDoAlreadyDependantWarning && (bChildIsAlreadyDependant || bChildHadParentAlready))
		{
			UE_LOGF(LogReplicationGraph, Warning, "FGlobalActorReplicationInfoMap::AddDependentActor child %ls already dependant of parent %ls", *GetNameSafe(Child), *GetNameSafe(Parent));
		}
	}
}

FName FNewReplicatedActorInfo::GetStreamingLevelNameOfActor(const AActor* Actor)
{
	ULevel* Level = Actor ? Cast<ULevel>(Actor->GetOuter()) : nullptr;
	return (Level && Level->IsPersistentLevel() == false) ? Level->GetOutermost()->GetFName() : NAME_None;
}

FActorConnectionPair::FActorConnectionPair(AActor* InActor, UNetConnection* InConnection)
	: Actor(InActor)
	, Connection(InConnection)
{
}

bool FActorRepListStatCollector::WasNodeVisited(const UReplicationGraphNode* NodeToVisit)
{
	const bool* Visited = VisitedNodes.Find(NodeToVisit);
	return (Visited != nullptr);
}

void FActorRepListStatCollector::FlagNodeVisited(const UReplicationGraphNode* NodeToVisit)
{
	VisitedNodes.Add(NodeToVisit, true);
}

void FActorRepListStatCollector::VisitRepList(const UReplicationGraphNode* NodeToVisit, const FActorRepListRefView& RepList)
{
	if (WasNodeVisited(NodeToVisit))
	{
		return;
	}

	FRepListStats& ClassStats = PerClassStats.FindOrAdd(NodeToVisit->GetClass()->GetFName());

	ClassStats.NumLists++;
	const uint32 ListSize = (uint32)RepList.RepList.Num();
	ClassStats.NumActors += ListSize;
	ClassStats.NumSlack += RepList.RepList.GetSlack();
	ClassStats.NumBytes += RepList.RepList.GetAllocatedSize();
	ClassStats.MaxListSize = FMath::Max(ClassStats.MaxListSize, ListSize);
}

void FActorRepListStatCollector::VisitStreamingLevelCollection(const UReplicationGraphNode* NodeToVisit, const FStreamingLevelActorListCollection& StreamingLevelList)
{
	if (WasNodeVisited(NodeToVisit))
	{
		return;
	}

	FRepListStats& ClassStats = PerClassStats.FindOrAdd(NodeToVisit->GetClass()->GetFName());

	// Collect the StreamingLevel stats
	for (const FStreamingLevelActorListCollection::FStreamingLevelActors& LevelList : StreamingLevelList.StreamingLevelLists)
	{
		const uint32 ListSize = (uint32)LevelList.ReplicationActorList.RepList.Num();
		const uint32 ListSlack = LevelList.ReplicationActorList.RepList.GetSlack();
		const SIZE_T ListBytes = LevelList.ReplicationActorList.RepList.GetAllocatedSize();

		{
			FRepListStats& StreamingLevelStats = PerStreamingLevelStats.FindOrAdd(LevelList.StreamingLevelName);

			StreamingLevelStats.NumLists++;
			StreamingLevelStats.NumActors += ListSize;
			StreamingLevelStats.NumSlack += ListSlack;
			StreamingLevelStats.NumBytes += ListBytes;
			StreamingLevelStats.MaxListSize = FMath::Max(StreamingLevelStats.MaxListSize, ListSize);
		}

		{
			ClassStats.NumLists++;
			ClassStats.NumActors += ListSize;
			ClassStats.NumSlack += ListSlack;
			ClassStats.NumBytes += ListBytes;
			ClassStats.MaxListSize = FMath::Max(ClassStats.MaxListSize, ListSize);
		}
	}
}

void FActorRepListStatCollector::VisitExplicitStreamingLevelList(FName ListOwnerName, FName StreamingLevelName, const FActorRepListRefView& RepList)
{
	const uint32 ListSize = (uint32)RepList.RepList.Num();
	const uint32 ListSlack = RepList.RepList.GetSlack();
	const SIZE_T ListBytes = RepList.RepList.GetAllocatedSize();

	FRepListStats& ClassStats = PerClassStats.FindOrAdd(ListOwnerName);
	ClassStats.NumLists++;
	ClassStats.NumActors += ListSize;
	ClassStats.NumSlack += ListSlack;
	ClassStats.NumBytes += ListBytes;
	ClassStats.MaxListSize = FMath::Max(ClassStats.MaxListSize, ListSize);
	
	FRepListStats& StreamingLevelStats = PerStreamingLevelStats.FindOrAdd(StreamingLevelName);
	StreamingLevelStats.NumLists++;
	StreamingLevelStats.NumActors += ListSize;
	StreamingLevelStats.NumSlack += ListSlack;
	StreamingLevelStats.NumBytes += ListBytes;
	StreamingLevelStats.MaxListSize = FMath::Max(StreamingLevelStats.MaxListSize, ListSize);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TArray<FActorRepListConstView>& FGatheredReplicationActorLists::GetLists(EActorRepListTypeFlags ListFlags) const
{
	// Synthesize an array of views of actors for backwards compatibility.
	// All actors appear in a single FActorRepListConstView.
	static FActorRepListRefView ActorListView;
	ActorListView.Reset();

	const TArrayView<const FActorRepListType> Actors = ViewActors(ListFlags);
	ActorListView.Reserve(Actors.Num());
	for (const FActorRepListType& Actor : Actors)
	{
		ActorListView.Add(Actor);
	}

	static TArray<FActorRepListConstView> ArrayOfViewsOfActors;
	ArrayOfViewsOfActors.Reset();
	ArrayOfViewsOfActors.Reserve(1);
	ArrayOfViewsOfActors.Add(FActorRepListConstView(ActorListView));

	return ArrayOfViewsOfActors;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FLevelBasedActorList::AddNetworkActor(AActor* NetActor)
{
	FNewReplicatedActorInfo ActorInfo(NetActor);

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		PermanentLevelActors.Add(NetActor);
	}
	else
	{
		StreamingLevelActors.AddActor(ActorInfo);
	}
}

bool FLevelBasedActorList::RemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo)
{
	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		return PermanentLevelActors.RemoveFast(ActorInfo.Actor);
	}
	else
	{
		return StreamingLevelActors.RemoveActorFast(ActorInfo);
	}
}

bool FLevelBasedActorList::RemoveNetworkActorOrdered(AActor* NetActor)
{
	FNewReplicatedActorInfo ActorInfo(NetActor);

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		return PermanentLevelActors.RemoveSlow(NetActor);
	}
	else
	{
		return StreamingLevelActors.RemoveActor(ActorInfo, false);
	}
}

bool FLevelBasedActorList::Contains(AActor* NetActor) const
{
	FNewReplicatedActorInfo ActorInfo(NetActor);

	if (ActorInfo.StreamingLevelName == NAME_None)
	{
		return PermanentLevelActors.Contains(NetActor);
	}
	else
	{
		return StreamingLevelActors.Contains(ActorInfo);
	}
}

void FLevelBasedActorList::Reset()
{
	PermanentLevelActors.Reset();
	StreamingLevelActors.Reset();
}

void FLevelBasedActorList::Gather(const FConnectionGatherActorListParameters& Params) const
{
	Params.OutGatheredReplicationLists.AddReplicationActorList(PermanentLevelActors);
	StreamingLevelActors.Gather(Params);
}

void FLevelBasedActorList::Gather(const UNetReplicationGraphConnection& ConnectionManager, FGatheredReplicationActorLists& OutGatheredList) const
{
	OutGatheredList.AddReplicationActorList(PermanentLevelActors);
	StreamingLevelActors.Gather(ConnectionManager, OutGatheredList);
}

void FLevelBasedActorList::AppendAllLists(FGatheredReplicationActorLists& OutGatheredList) const
{
	OutGatheredList.AddReplicationActorList(PermanentLevelActors);
	StreamingLevelActors.AppendAllLists(OutGatheredList);
}

void FLevelBasedActorList::GetAllActors(TArray<AActor*>& OutAllActors) const
{
#if !UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY
	PermanentLevelActors.AppendToTArray(OutAllActors);
	StreamingLevelActors.GetAll_Debug(OutAllActors);
#else
	TArray<FActorRepListType> AllActors;
	PermanentLevelActors.AppendToTArray(AllActors);
	StreamingLevelActors.GetAll_Debug(AllActors);

	for (const FActorRepListType& RepListActor : AllActors)
	{
		OutAllActors.Add(RepListActor);
	}
#endif
}

void FLevelBasedActorList::CountBytes(FArchive& Ar) const
{
	PermanentLevelActors.CountBytes(Ar);
	StreamingLevelActors.CountBytes(Ar);
}

void FLevelBasedActorList::UpdateActorLevel(AActor* NetActor, FName PreviousStreamingLevelName)
{
	bool bRemovedActor = false;

	if (PreviousStreamingLevelName == NAME_None)
	{
		bRemovedActor = PermanentLevelActors.RemoveFast(NetActor);
	}
	else
	{
		bRemovedActor = StreamingLevelActors.RemoveActorFromLevelFast(NetActor, PreviousStreamingLevelName);
	}

	if (bRemovedActor)
	{
		AddNetworkActor(NetActor);
	}
	else
	{
		UE_LOGF(LogReplicationGraph, Warning, "FLevelBasedActorList::UpdateActorLevel did not find existing actor %ls in level lists", *GetFullNameSafe(NetActor));
	}
}