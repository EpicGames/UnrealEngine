// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReplicationGraph.h"
#include "BasicReplicationGraph.generated.h"

struct FNewReplicatedActorInfo;
#if WITH_GAMEPLAY_DEBUGGER
class AGameplayDebuggerCategoryReplicator;
#endif

USTRUCT()
struct FConnectionAlwaysRelevantNodePair
{
	GENERATED_BODY()
	FConnectionAlwaysRelevantNodePair() { }
	FConnectionAlwaysRelevantNodePair(UNetConnection* InConnection, UReplicationGraphNode_AlwaysRelevant_ForConnection* InNode) : NetConnection(InConnection), Node(InNode) { }
	bool operator==(UNetConnection* InConnection) const;

	UPROPERTY()
	TObjectPtr<UNetConnection> NetConnection = nullptr;

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode_AlwaysRelevant_ForConnection> Node = nullptr;	
};


/** 
 * A basic implementation of replication graph. It only supports NetCullDistanceSquared, bAlwaysRelevant, bOnlyRelevantToOwner. These values cannot change per-actor at runtime. 
 * This is meant to provide a simple example implementation. More robust implementations will be required for more complex games. ShootGame is another example to check out.
 * 
 * To enable this via ini:
 * [/Script/OnlineSubsystemUtils.IpNetDriver]
 * ReplicationDriverClassName="/Script/ReplicationGraph.BasicReplicationGraph"
 * 
 **/
UCLASS(transient, config=Engine)
class UBasicReplicationGraph :public UReplicationGraph
{
	GENERATED_BODY()

public:

	UBasicReplicationGraph();

	virtual void InitGlobalActorClassSettings() override;
	virtual void InitGlobalGraphNodes() override;
	virtual void InitConnectionGraphNodes(UNetReplicationGraphConnection* RepGraphConnection) override;
	virtual void RouteAddNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo) override;
	virtual void RouteRemoveNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo) override;

protected:
	virtual void RouteRenameNetworkActorToNodes(const FRenamedReplicatedActorInfo& ActorInfo) override;

public:
	virtual int32 ServerReplicateActors(float DeltaSeconds) override;

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode_GridSpatialization2D> GridNode;

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode_ActorList> AlwaysRelevantNode;

	UPROPERTY()
	TArray<FConnectionAlwaysRelevantNodePair> AlwaysRelevantForConnectionList;

	/** Actors that are only supposed to replicate to their owning connection, but that did not have a connection on spawn */
	UPROPERTY()
	TArray<TObjectPtr<AActor>> ActorsWithoutNetConnection;

	UReplicationGraphNode_AlwaysRelevant_ForConnection* GetAlwaysRelevantNodeForConnection(UNetConnection* Connection);

#if WITH_GAMEPLAY_DEBUGGER
	void OnGameplayDebuggerOwnerChange(AGameplayDebuggerCategoryReplicator* Debugger, APlayerController* OldOwner);
#endif
};
