// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================
Replication Graph

Implementation of Replication Driver. This is customizable via subclassing UReplicationGraph. Default implementation (UReplicationGraph) does not fully function and is intended to be overridden.
	Check out BasicReplicationGraph.h for a minimal implementation that works "out of the box" with a minimal feature set.
	Check out ShooterGame / UShooterReplicationGraph for a more advanced implementation.



High level overview of ReplicationGraph:
	* The graph is a collection of nodes which produce replication lists for each network connection. The graph essentially maintains persistent lists of actors to replicate and feeds them to connections. 
	
	* This allows for more work to be shared and greatly improves the scalability of the system with respect to number of actors * number of connections.
	
	* For example, one node on the graph is the spatialization node. All actors that essentially use distance based relevancy will go here. There are also always relevant nodes. Nodes can be global, per connection, or shared (E.g, "Always relevant for team" nodes).

	* The main impact here is that virtual functions like IsNetRelevantFor and GetNetPriority are not used by the replication graph. Several properties are also not used or are slightly different in use. 

	* Instead there are essentially three ways for game code to affect this part of replication:
		* The graph itself. Adding new UReplicationNodes or changing the way an actor is placed in the graph.
		* FGlobalActorReplicationInfo: The associative data the replication graph keeps, globally, about each actor. 
		* FConnectionReplicationActorInfo: The associative data that the replication keeps, per connection, about each actor. 

	* After returned from the graph, the actor lists are further culled for distance and frequency, then merged and prioritized. The end result is a sorted list of actors to replicate that we then do logic for creating or updating actor channels.



Subclasses should implement these functions:

UReplicationGraph::InitGlobalActorClassSettings
	Initialize UReplicationGraph::GlobalActorReplicationInfoMap.

UReplicationGraph::InitGlobalGraphNodes
	Instantiate new UGraphNodes via ::CreateNewNode. Use ::AddGlobalGraphNode if they are global (for all connections).

UReplicationGraph::RouteAddNetworkActorToNodes/::RouteRemoveNetworkActorToNodes
	Route actor spawning/despawning to the right node. (Or your nodes can gather the actors themselves)

UReplicationGraph::InitConnectionGraphNodes
	Initialize per-connection nodes (or associate shared nodes with them via ::AddConnectionGraphNode)


=============================================================================*/

#pragma once

#include "Engine/NetDriver.h"
#include "Engine/ReplicationDriver.h"
#include "Engine/World.h"
#include "ReplicationGraphTypes.h"

#include "UObject/Package.h"
#include "ReplicationGraph.generated.h"

#define UE_API REPLICATIONGRAPH_API

struct FReplicationGraphDestructionSettings;

typedef TObjectKey<class UNetReplicationGraphConnection> FRepGraphConnectionKey;

#ifndef DO_ENABLE_REPGRAPH_DEBUG_ACTOR
#define DO_ENABLE_REPGRAPH_DEBUG_ACTOR !(UE_BUILD_SHIPPING)
#endif // DO_ENABLE_REPGRAPH_DEBUG_ACTOR

UCLASS(MinimalAPI, abstract, transient, config=Engine)
class UReplicationGraphNode : public UObject
{
	GENERATED_BODY()

public:

	UE_API UReplicationGraphNode();

	UE_API virtual void Serialize(FArchive& Ar) override;

	/** Called when a network actor is spawned or an actor changes replication status */
	virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& Actor ) PURE_VIRTUAL(UReplicationGraphNode::NotifyAddNetworkActor, );
	
	/** Called when a networked actor is being destroyed or no longer wants to replicate */
	virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& Actor, bool bWarnIfNotFound=true) PURE_VIRTUAL(UReplicationGraphNode::NotifyRemoveNetworkActor, return false; );

	/**
	 * If Net.RepGraph.HandleDynamicActorRename is enabled, this is called when a dynamic actor is changing its outer.
	 * If the node maintains level information for actors, and if replicated actors may be renamed, the implementation needs to update its level info.
	 * Returns true if the actor was found & updated.
	 */
	virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& Actor, bool bWarnIfNotFound=true) PURE_VIRTUAL(UReplicationGraphNode::NotifyActorRenamed, return false; );

	/** Called when world changes or when all subclasses should dump any persistent data/lists about replicated actors here. (The new/next world will be set before this is called) */
	UE_API virtual void NotifyResetAllNetworkActors();
	
	/** Mark the node and all its children PendingKill */
	UE_API virtual void TearDown();
	
	virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) PURE_VIRTUAL(UReplicationGraphNode::GatherActorListsForConnection, );

	/** Called once per frame prior to replication ONLY on root nodes (nodes created via UReplicationGraph::CreateNode) has RequiresPrepareForReplicationCall=true */
	virtual void PrepareForReplication() { };

	/** Debugging only function to return a normal TArray of actor rep list (for logging, debug UIs, etc) */
	UE_API virtual void GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const;

	/** Debugging only function to verify referenced actors (if any) are still valid. */
	UE_API void VerifyActorReferences();

	// -----------------------------------------------------

	bool GetRequiresPrepareForReplication() const { return bRequiresPrepareForReplicationCall; }

	void Initialize(const TSharedPtr<FReplicationGraphGlobalData>& InGraphGlobals) { GraphGlobals = InGraphGlobals; }

	virtual UWorld* GetWorld() const override final { return  GraphGlobals.IsValid() ? GraphGlobals->World : nullptr; }

	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const;

	virtual FString GetDebugString() const { return GetName(); }

	UE_API void DoCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const;

	/** Allocates and initializes ChildNode of a specific type T. This is what you will want to call in your FCreateChildNodeFuncs.  */
	template< class T >
	T* CreateChildNode()
	{
		T* NewNode = NewObject<T>(this);
		NewNode->Initialize(GraphGlobals);
		AllChildNodes.Add(NewNode);
		return NewNode;
	}

	/** Allocates and initializes ChildNode of a specific type T with a given base name. This is what you will want to call in your FCreateChildNodeFuncs.  */
	template< class T >
	T* CreateChildNode(FName NodeBaseName)
	{
		FName UniqueNodeName = MakeUniqueObjectName(this, T::StaticClass(), NodeBaseName);
		T* NewNode = NewObject<T>(this, UniqueNodeName);
		NewNode->Initialize(GraphGlobals);
		AllChildNodes.Add(NewNode);
		return NewNode;
	}


	/** Node removal behavior */
	enum class NodeOrdering
	{
		IgnoreOrdering, // Use faster removal but may break node processing order
		KeepOrder,		// Use slower removal but keep the node order intact
	};

	/** Remove a child node from our list and flag it for destruction. Returns if the node was found or not */
	UE_API bool RemoveChildNode(UReplicationGraphNode* OutChildNode, UReplicationGraphNode::NodeOrdering NodeOrder=UReplicationGraphNode::NodeOrdering::IgnoreOrdering);

	/** Remove all null and about to be destroyed nodes from our list */
	UE_API void CleanChildNodes(UReplicationGraphNode::NodeOrdering NodeOrder);

protected:

	/**
	 * Implement this to visit any FActorRepListRefView and FStreamingLevelActorListCollection your node implemented.
	 */
	virtual void OnCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const {}


	/**
	 * Verify the validity of any potentially stale actor pointers. Used for debugging.
	 */
	virtual void VerifyActorReferencesInternal() {}

	UE_API bool VerifyActorReference(const FActorRepListType& Actor) const;

protected:

	UPROPERTY()
	TArray< TObjectPtr<UReplicationGraphNode> > AllChildNodes;

	TSharedPtr<FReplicationGraphGlobalData>	GraphGlobals;

	/** Determines if PrepareForReplication() is called. This currently must be set in the constructor, not dynamically. */
	bool bRequiresPrepareForReplicationCall = false;
};

// -----------------------------------

/** A Node that contains ReplicateActorLists. This contains 1 "base" list and a TArray of lists that are conditioned on a streaming level being loaded. */
UCLASS(MinimalAPI)
class UReplicationGraphNode_ActorList : public UReplicationGraphNode
{
	GENERATED_BODY()

public:
	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo) override;
	
	UE_API virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;
	
	UE_API virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;

	UE_API virtual void NotifyResetAllNetworkActors() override;
	
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;

	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const override;

	UE_API virtual void GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const override;

	UE_API virtual void VerifyActorReferencesInternal() override;

	UE_API virtual void TearDown() override;

	/** Removes the actor very quickly but breaks the list order */
	UE_API bool RemoveNetworkActorFast(const FNewReplicatedActorInfo& ActorInfo);

	/** Copies the contents of Source into this node. Note this does not copy child nodes, just the ReplicationActorList/StreamingLevelCollection lists on this node. */
	UE_API void DeepCopyActorListsFrom(const UReplicationGraphNode_ActorList* Source);

protected:

	/** Just logs our ReplicationActorList and StreamingLevelCollection (not our child nodes). Useful when subclasses override LogNode */
	UE_API void LogActorList(FReplicationGraphDebugInfo& DebugInfo) const;

	UE_API virtual void OnCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const override;

	/** The base list that most actors will go in */
	FActorRepListRefView ReplicationActorList;

	/** A collection of lists that streaming actors go in */
	FStreamingLevelActorListCollection StreamingLevelCollection;

	friend class AReplicationGraphDebugActor;
};

// -----------------------------------

/** A Node that contains ReplicateActorLists. This contains multiple buckets for non streaming level actors and will pull from each bucket on alternating frames. It is a way to broadly load balance. */
UCLASS(MinimalAPI)
class UReplicationGraphNode_ActorListFrequencyBuckets : public UReplicationGraphNode
{
	GENERATED_BODY()

public:

	struct FSettings
	{
		int32 NumBuckets = 3;
		int32 ListSize = 12;
		bool EnableFastPath = false; // Whether to return lists as FastPath in "off frames". Defaults to false.
		int32 FastPathFrameModulo = 1; // Only do fast path if frame num % this = 0

		// Threshold for dynamically balancing buckets bsaed on number of actors in this node. E.g, more buckets when there are more actors.
		struct FBucketThresholds
		{
			FBucketThresholds(int32 InMaxActors, int32 InNumBuckets) : MaxActors(InMaxActors), NumBuckets(InNumBuckets) { }
			int32 MaxActors;	// When num actors <= to MaxActors
			int32 NumBuckets;	// use this NumBuckets
		};

		TArray<FBucketThresholds, TInlineAllocator<4>> BucketThresholds;

		void CountBytes(FArchive& Ar) const 
		{
			BucketThresholds.CountBytes(Ar);
		}
	};

	/** Default settings for all nodes. By being static, this allows games to easily override the settings are all nodes without having to subclass every graph node class */
	static UE_API FSettings DefaultSettings;

	/** Settings for this specific node. If not set we will fallback to the static/global DefaultSettings */
	TSharedPtr<FSettings> Settings;

	const FSettings& GetSettings() const { return Settings.IsValid() ? *Settings.Get() : DefaultSettings; }
	
	UReplicationGraphNode_ActorListFrequencyBuckets() { if (!HasAnyFlags(RF_ClassDefaultObject)) { SetNonStreamingCollectionSize(GetSettings().NumBuckets); } }

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo) override;
	
	UE_API virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;
	
	UE_API virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;

	UE_API virtual void NotifyResetAllNetworkActors() override;
	
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;

	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const override;

	UE_API virtual void TearDown() override;

	UE_API virtual void GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const override;

	UE_API virtual void VerifyActorReferencesInternal() override;

	UE_API virtual void OnCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const override;

	UE_API void SetNonStreamingCollectionSize(const int32 NewSize);

protected:

	UE_API void CheckRebalance();

	int32 TotalNumNonStreamingActors = 0;

	/** Non streaming actors go in one of these lists */
	TArray<FActorRepListRefView, TInlineAllocator<2>> NonStreamingCollection;

	/** A collection of lists that streaming actors go in */
	FStreamingLevelActorListCollection StreamingLevelCollection;

	friend class AReplicationGraphDebugActor;
};

// -----------------------------------


/** A node intended for dynamic (moving) actors where replication frequency is based on distance to the connection's view location */
UCLASS(MinimalAPI)
class UReplicationGraphNode_DynamicSpatialFrequency : public UReplicationGraphNode_ActorList
{
	GENERATED_BODY()

public:

	UE_API UReplicationGraphNode_DynamicSpatialFrequency();
	
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;
	
	UE_API virtual void VerifyActorReferencesInternal() override;

	// --------------------------------------------------------

	struct FSpatializationZone
	{
		// Init directly
		FSpatializationZone(float InMinDotProduct, float InMinDistPct, float InMaxDistPct, uint32 InMinRepPeriod, uint32 InMaxRepPeriod, uint32 InMinRepPeriod_FastPath, uint32 InMaxRepPeriod_FastPath) :
			MinDotProduct(InMinDotProduct), MinDistPct(InMinDistPct), MaxDistPct(InMaxDistPct), MinRepPeriod(InMinRepPeriod), MaxRepPeriod(InMaxRepPeriod), FastPath_MinRepPeriod(InMinRepPeriod_FastPath), FastPath_MaxRepPeriod(InMaxRepPeriod_FastPath) { }

		// Init based on target frame rate
		FSpatializationZone(float InMinDotProduct, float InMinDistPct, float InMaxDistPct, float InMinRepHz, float InMaxRepHz, float InMinRepHz_FastPath, float InMaxRepHz_FastPath, float TickRate) :
			MinDotProduct(InMinDotProduct), MinDistPct(InMinDistPct), MaxDistPct(InMaxDistPct), 
				MinRepPeriod(HzToFrm(InMinRepHz,TickRate)), MaxRepPeriod(HzToFrm(InMaxRepHz,TickRate)), FastPath_MinRepPeriod(HzToFrm(InMinRepHz_FastPath,TickRate)), FastPath_MaxRepPeriod(HzToFrm(InMaxRepHz_FastPath,TickRate)) { }

		float MinDotProduct = 1.f;	// Must have dot product >= to this to be in this zone
		float MinDistPct = 0.f;		// Min distance as pct of per-connection culldistance, to map to MinRepPeriod
		float MaxDistPct = 1.f;		// Max distance as pct of per-connection culldistance, to map to MaxRepPeriod

		uint32 MinRepPeriod = 5;
		uint32 MaxRepPeriod = 10;

		uint32 FastPath_MinRepPeriod = 1;
		uint32 FastPath_MaxRepPeriod = 5;

		static uint32 HzToFrm(float Hz, float TargetFrameRate)
		{
			return  Hz > 0.f ? (uint32)FMath::CeilToInt(TargetFrameRate / Hz) : 0;
		}
	};

	struct FSettings
	{
		FSettings() :MaxBitsPerFrame(0), MaxNearestActors(-1) { }

		FSettings(TArrayView<FSpatializationZone> InZoneSettings, TArrayView<FSpatializationZone> InZoneSettings_NonFastSharedActors, int64 InMaxBitsPerFrame, int32 InMaxNearestActors = -1) 
			: ZoneSettings(InZoneSettings), ZoneSettings_NonFastSharedActors(InZoneSettings_NonFastSharedActors), MaxBitsPerFrame(InMaxBitsPerFrame), MaxNearestActors(InMaxNearestActors) { }

		TArrayView<FSpatializationZone> ZoneSettings;
		TArrayView<FSpatializationZone> ZoneSettings_NonFastSharedActors;	// Zone Settings for actors that do not support FastShared replication.
		int64 MaxBitsPerFrame;
		int32 MaxNearestActors; // Only replicate the X nearest actors to a connection in this node. -1 = no limit.
	};

	static UE_API FSettings DefaultSettings;	// Default settings used by all instance
	FSettings* Settings;				// per instance override settings (optional)

	FSettings& GetSettings() { return Settings ? *Settings : DefaultSettings; }

	const char* CSVStatName; // Statname to use for tracking the gather/prioritizing phase of this node

protected:

	struct FDynamicSpatialFrequency_SortedItem
	{
		FDynamicSpatialFrequency_SortedItem() { }
		FDynamicSpatialFrequency_SortedItem(AActor* InActor, int32 InFramesTillReplicate, bool InEnableFastPath, FGlobalActorReplicationInfo* InGlobal, FConnectionReplicationActorInfo* InConnection ) 
			: Actor(InActor), FramesTillReplicate(InFramesTillReplicate), EnableFastPath(InEnableFastPath), GlobalInfo(InGlobal), ConnectionInfo(InConnection) { }

		FDynamicSpatialFrequency_SortedItem(AActor* InActor, int32 InDistance, FGlobalActorReplicationInfo* InGlobal)
			: Actor(InActor), FramesTillReplicate(InDistance), GlobalInfo(InGlobal) {}

		bool operator<(const FDynamicSpatialFrequency_SortedItem& Other) const { return FramesTillReplicate < Other.FramesTillReplicate; }

		AActor* Actor = nullptr;

		int32 FramesTillReplicate = 0; // Note this also serves as "Distance Sq" when doing the FSettings::MaxNearestActors pass.
		bool EnableFastPath = false;

		FGlobalActorReplicationInfo* GlobalInfo = nullptr;
		FConnectionReplicationActorInfo* ConnectionInfo = nullptr;
	};

	// Working area for our sorted replication list. Reset each frame for each connection as we build the list
	TArray<FDynamicSpatialFrequency_SortedItem>	SortedReplicationList;
	
	// Working ints for adaptive load balancing. Does not count actors that rep every frame
	int32 NumExpectedReplicationsThisFrame = 0;
	int32 NumExpectedReplicationsNextFrame = 0;

	bool IgnoreCullDistance = false;

	UE_API virtual void GatherActors(const FActorRepListRefView& RepList, FGlobalActorReplicationInfoMap& GlobalMap, FPerConnectionActorInfoMap& ConnectionMap, const FConnectionGatherActorListParameters& Params, UNetConnection* NetConnection);
	UE_API virtual void GatherActors_DistanceOnly(const FActorRepListRefView& RepList, FGlobalActorReplicationInfoMap& GlobalMap, FPerConnectionActorInfoMap& ConnectionMap, const FConnectionGatherActorListParameters& Params);

	UE_DEPRECATED(5.2, "This function was replaced with a version that needs to receive a UNetReplicationGraphConnection instead of a NetConnection")
	UE_API void CalcFrequencyForActor(AActor* Actor, UReplicationGraph* RepGraph, UNetConnection* NetConnection, FGlobalActorReplicationInfo& GlobalInfo, FPerConnectionActorInfoMap& ConnectionMap, FSettings& MySettings, const FNetViewerArray& Viewers, const uint32 FrameNum, int32 ExistingItemIndex);

	UE_API void CalcFrequencyForActor(AActor* Actor, UReplicationGraph* RepGraph, const UNetReplicationGraphConnection& RepGraphConnection, FGlobalActorReplicationInfo& GlobalInfo, FPerConnectionActorInfoMap& ConnectionMap, FSettings& MySettings, const FNetViewerArray& Viewers, const uint32 FrameNum, int32 ExistingItemIndex);
};


// -----------------------------------

/** Removes dormant (on connection) actors from its rep lists */
UCLASS(MinimalAPI)
class UReplicationGraphNode_ConnectionDormancyNode : public UReplicationGraphNode_ActorList
{
	GENERATED_BODY()
public:

	/** 
	* This setting determines the number of frames from the last Gather call when we consider a node obsolete.
	* The default value of 0 disables this condition.
	* Enable this to reduce the memory footprint of the RepGraph and optimize the exponential cost of dormancy events (the more each connection moves around the grid cell the more each dormancy events is listened to).
	* Inversely if clients go back and forth across the same cell multiple times the cost of the first Gather of newly created cell is very high (it rechecks all dormant actors in the cell before removing them).
	* This means this optimization is ideal only if you know the clients can't go back to previously visited cells, or that they do so very infrequently.
	*/
	static UE_API void SetNumFramesUntilObsolete(uint32 NumFrames);

public:

	void InitConnectionNode(const FRepGraphConnectionKey& InConnectionOwner, uint32 CurrentFrame) 
	{
		ConnectionOwner = InConnectionOwner;
		LastGatheredFrame = CurrentFrame;
	}

	UE_API virtual void TearDown() override;

	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;
	UE_API virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool WarnIfNotFound) override;
	UE_API virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;
	UE_API virtual void NotifyResetAllNetworkActors() override;
	UE_API virtual void VerifyActorReferencesInternal() override;

	UE_API void NotifyActorDormancyFlush(FActorRepListType Actor);

	UE_API void OnClientVisibleLevelNameAdd(FName LevelName, UWorld* World);

	UE_API bool IsNodeObsolete(uint32 CurrentFrame) const;

private:

	static UE_API uint32 NumFramesUntilObsolete;

private:

	FRepGraphConnectionKey ConnectionOwner;

	uint32 LastGatheredFrame = 0;

	UE_API void ConditionalGatherDormantActorsForConnection(FActorRepListRefView& ConnectionRepList, const FConnectionGatherActorListParameters& Params, FActorRepListRefView* RemovedList);
	
	int32 TrickleStartCounter = 10;

	// Tracks actors we've removed in this per-connection node, so that we can restore them if the streaming level is unloaded and reloaded.
	FStreamingLevelActorListCollection RemovedStreamingLevelActorListCollection;
};


/** Stores per-connection copies of a main actor list. Skips and removes elements from per connection list that are fully dormant */
UCLASS(MinimalAPI)
class UReplicationGraphNode_DormancyNode : public UReplicationGraphNode_ActorList
{
	GENERATED_BODY()

public:

	/** Connection Z location has to be < this for ConnectionsNodes to be made. */
	static UE_API FVector::FReal MaxZForConnection; 

public:

	virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo) override { ensureMsgf(false, TEXT("UReplicationGraphNode_DormancyNode::NotifyAddNetworkActor not functional.")); }
	virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override { ensureMsgf(false, TEXT("UReplicationGraphNode_DormancyNode::NotifyRemoveNetworkActor not functional.")); return false; }
	virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override { ensureMsgf(false, TEXT("UReplicationGraphNode_DormancyNode::NotifyActorRenamed not functional.")); return false; }
	UE_API virtual void NotifyResetAllNetworkActors() override;
	
	UE_API void AddDormantActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo);
	UE_API void RemoveDormantActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo);
	UE_API void RenameDormantActor(const FRenamedReplicatedActorInfo& ActorInfo);

	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;

	UE_API void OnActorDormancyFlush(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo);

	UE_API void ConditionalGatherDormantDynamicActors(FActorRepListRefView& RepList, const FConnectionGatherActorListParameters& Params, FActorRepListRefView* RemovedList, bool bEnforceReplistUniqueness = false, FActorRepListRefView* RemoveFromList = nullptr);

	UE_API UReplicationGraphNode_ConnectionDormancyNode* GetExistingConnectionNode(const FConnectionGatherActorListParameters& Params);

	UE_API UReplicationGraphNode_ConnectionDormancyNode* GetConnectionNode(const FConnectionGatherActorListParameters& Params);

private:

	UE_API UReplicationGraphNode_ConnectionDormancyNode* CreateConnectionNode(const FConnectionGatherActorListParameters& Params);

	/** Function called on every ConnectionDormancyNode in our list */
	typedef TFunction<void(UReplicationGraphNode_ConnectionDormancyNode*)> FConnectionDormancyNodeFunction;

	/**
	 * Iterates over all ConnectionDormancyNodes and calls the function on those still valid.
	 * If the node is considered obsolete, the function will destroy or skip over the ConnectionDormancyNode 
	 */
	UE_API void CallFunctionOnValidConnectionNodes(FConnectionDormancyNodeFunction Function);

private:

	typedef TSortedMap<FRepGraphConnectionKey, UReplicationGraphNode_ConnectionDormancyNode*> FConnectionDormancyNodeMap;
	FConnectionDormancyNodeMap ConnectionNodes;
};

UCLASS(MinimalAPI)
class UReplicationGraphNode_GridCell : public UReplicationGraphNode_ActorList
{
	GENERATED_BODY()

public:

	virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo) override { ensureMsgf(false, TEXT("UReplicationGraphNode_Simple2DSpatializationLeaf::NotifyAddNetworkActor not functional.")); }
	virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override { ensureMsgf(false, TEXT("UReplicationGraphNode_Simple2DSpatializationLeaf::NotifyRemoveNetworkActor not functional.")); return false; }
	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const override;
	UE_API virtual void GetAllActorsInNode_Debugging(TArray<FActorRepListType>& OutArray) const override;

	UE_API void AddStaticActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalRepInfo, bool bParentNodeHandlesDormancyChange);
	UE_API void AddDynamicActor(const FNewReplicatedActorInfo& ActorInfo);

	UE_API void RemoveStaticActor(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bWasAddedAsDormantActor);
	UE_API void RemoveDynamicActor(const FNewReplicatedActorInfo& ActorInfo);

	UE_API void RenameStaticActor(const FRenamedReplicatedActorInfo& ActorInfo, bool bWasAddedAsDormantActor);
	UE_API void RenameDynamicActor(const FRenamedReplicatedActorInfo& ActorInfo);

	// Allow graph to override function for creating the dynamic node in the cell
	TFunction<UReplicationGraphNode*(UReplicationGraphNode_GridCell* Parent)> CreateDynamicNodeOverride;

	UE_API UReplicationGraphNode_DormancyNode* GetDormancyNode(bool bInCreateIfMissing = true);

private:

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode> DynamicNode = nullptr;

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode_DormancyNode> DormancyNode = nullptr;

	UE_API UReplicationGraphNode* GetDynamicNode();

	UE_API void OnActorDormancyFlush(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo, UReplicationGraphNode_DormancyNode* DormancyNode );

	UE_API void ConditionalCopyDormantActors(FActorRepListRefView& FromList, UReplicationGraphNode_DormancyNode* ToNode);	
	UE_API void OnStaticActorNetDormancyChange(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo, ENetDormancy NewVlue, ENetDormancy OldValue);
};

// -----------------------------------

UCLASS(MinimalAPI)
class UReplicationGraphNode_GridSpatialization2D : public UReplicationGraphNode
{
	GENERATED_BODY()

public:

	UE_API UReplicationGraphNode_GridSpatialization2D();

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& Actor) override;	
	UE_API virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override;
	UE_API virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& Actor, bool bWarnIfNotFound=true) override;
	UE_API virtual void NotifyResetAllNetworkActors() override;
	UE_API virtual void PrepareForReplication() override;
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;
	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const override;
	UE_API virtual void VerifyActorReferencesInternal() override;

	void AddActor_Static(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo) { AddActorInternal_Static(ActorInfo, ActorRepInfo, false); }
	void AddActor_Dynamic(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo) { AddActorInternal_Dynamic(ActorInfo); }
	UE_API void AddActor_Dormancy(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo);

	UE_API void RemoveActor_Static(const FNewReplicatedActorInfo& ActorInfo);
	void RemoveActor_Dynamic(const FNewReplicatedActorInfo& ActorInfo) { RemoveActorInternal_Dynamic(ActorInfo); }
	UE_API void RemoveActor_Dormancy(const FNewReplicatedActorInfo& ActorInfo);

	UE_API void RenameActor_Static(const FRenamedReplicatedActorInfo& ActorInfo);
	UE_API void RenameActor_Dynamic(const FRenamedReplicatedActorInfo& ActorInfo);
	UE_API void RenameActor_Dormancy(const FRenamedReplicatedActorInfo& ActorInfo);

	// Called if cull distance changes. Note the caller must update Global/Connection actor rep infos. This just changes cached state within this node
	UE_API virtual void NotifyActorCullDistChange(AActor* Actor, FGlobalActorReplicationInfo& GlobalInfo, float OldDist);
	
	float		CellSize;
	FVector2D	SpatialBias;
	double		ConnectionMaxZ; // Connection locations have to be <= to this to pull from the grid

	/** When the GridBounds is set we limit the creation of cells to be exclusively inside the passed region.
	    Viewers who gather nodes outside this region will be clamped to the closest cell inside the box.
		Actors whose location is outside the box will be clamped to the closest cell inside the box.
	*/
	UE_API void SetBiasAndGridBounds(const FBox& GridBox);

	// Allow graph to override function for creating cell nodes in this grid.
	TFunction<UReplicationGraphNode_GridCell*(UReplicationGraphNode_GridSpatialization2D* Parent)>	CreateCellNodeOverride;

	void ForceRebuild() { bNeedsRebuild = true; }

	// Marks a class as preventing spatial rebuilds when an instance leaves the grid
	void AddToClassRebuildDenyList(UClass* Class) { ClassRebuildDenyList.Set(Class, true); }

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TArray<FString> DebugActorNames;
#endif

	// When enabled the RepGraph tells clients to destroy dormant dynamic actors when they go out of relevancy.
	bool bDestroyDormantDynamicActors = true;
	// How many frames dormant actors should be outside relevancy range to be destroyed
	int32 DestroyDormantDynamicActorsCellTTL = 1;
	// Optimization to spread cost over multiple frames
	int32 ReplicatedDormantDestructionInfosPerFrame = MAX_int32;

protected:

	/** For adding new actor to the graph */
	UE_API virtual void AddActorInternal_Dynamic(const FNewReplicatedActorInfo& ActorInfo);
	UE_API virtual void AddActorInternal_Static(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool IsDormancyDriven); // Can differ the actual call to implementation if actor is not ready
	UE_API virtual void AddActorInternal_Static_Implementation(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool IsDormancyDriven);

	UE_API virtual void RemoveActorInternal_Dynamic(const FNewReplicatedActorInfo& Actor);
	UE_API virtual void RemoveActorInternal_Static(const FNewReplicatedActorInfo& Actor, FGlobalActorReplicationInfo& ActorRepInfo, bool WasAddedAsDormantActor);

private:

	UE_API bool WillActorLocationGrowSpatialBounds(const FVector& Location) const;

	/** Called when an actor is out of spatial bounds */
	UE_API void HandleActorOutOfSpatialBounds(AActor* Actor, const FVector& Location3D, const bool bStaticActor);

	// Optional value to limit the grid to a specific region
	FBox GridBounds;

	// Classmap of actor classes which CANNOT force a rebuild of the spatialization tree. They will be clamped instead. E.g, projectiles.
	TClassMap<bool> ClassRebuildDenyList;
	
	struct FActorCellInfo
	{
		bool IsValid() const { return StartX != -1; }
		void Reset() { StartX = -1; }
		int32 StartX=-1;
		int32 StartY;
		int32 EndX;
		int32 EndY;
	};
	
	struct FCachedDynamicActorInfo
	{
		FCachedDynamicActorInfo(const FNewReplicatedActorInfo& InInfo) : ActorInfo(InInfo) { }
		FNewReplicatedActorInfo ActorInfo;	
		FActorCellInfo CellInfo;
	};
	
	TMap<FActorRepListType, FCachedDynamicActorInfo> DynamicSpatializedActors;


	struct FCachedStaticActorInfo
	{
		FCachedStaticActorInfo(const FNewReplicatedActorInfo& InInfo, bool bInDormDriven) : ActorInfo(InInfo), bDormancyDriven(bInDormDriven) { }
		FNewReplicatedActorInfo ActorInfo;
		bool bDormancyDriven = false; // This actor will be removed from static actor list if it becomes non dormant.
	};

	TMap<FActorRepListType, FCachedStaticActorInfo> StaticSpatializedActors;

	// Static spatialized actors that were not fully initialized when registering with this node. We differ their placement in the grid until the next frame
	struct FPendingStaticActors
	{
		FPendingStaticActors(const FActorRepListType& InActor, const bool InDormancyDriven) : Actor(InActor), DormancyDriven(InDormancyDriven) { }
		bool operator==(const FActorRepListType& InActor) const { return InActor == Actor; };
		FActorRepListType Actor;
		bool DormancyDriven;
	};
	TArray<FPendingStaticActors> PendingStaticSpatializedActors;

	UE_API void OnNetDormancyChange(FActorRepListType Actor, FGlobalActorReplicationInfo& GlobalInfo, ENetDormancy NewVlue, ENetDormancy OldValue);
	
	UE_API void PutStaticActorIntoCell(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& ActorRepInfo, bool bDormancyDriven);

	UReplicationGraphNode_GridCell* GetCellNode(UReplicationGraphNode_GridCell*& NodePtr)
	{
		if (NodePtr == nullptr)
		{
			if (CreateCellNodeOverride)
			{
				NodePtr = CreateCellNodeOverride(this);
			}
			else
			{
				NodePtr = CreateChildNode<UReplicationGraphNode_GridCell>();
			}
		}

		return NodePtr;
	}

	TArray< TArray<UReplicationGraphNode_GridCell*> > Grid;

	TArray<UReplicationGraphNode_GridCell*>& GetGridX(int32 X)
	{
		if (Grid.Num() <= X)
		{
			Grid.SetNum(X+1);
		}
		return Grid[X];
	}

	UReplicationGraphNode_GridCell*& GetCell(TArray<UReplicationGraphNode_GridCell*>& GridX, int32 Y)
	{
		if (GridX.Num() <= Y)
		{
			GridX.SetNum(Y+1);
		}
		return GridX[Y];
	}		

	UReplicationGraphNode_GridCell*& GetCell(int32 X, int32 Y)
	{
		TArray<UReplicationGraphNode_GridCell*>& GridX = GetGridX(X);
		return GetCell(GridX, Y);
	}

	bool bNeedsRebuild = false;

	UE_API void GetGridNodesForActor(FActorRepListType Actor, const FGlobalActorReplicationInfo& ActorRepInfo, TArray<UReplicationGraphNode_GridCell*>& OutNodes);
	UE_API void GetGridNodesForActor(FActorRepListType Actor, const UReplicationGraphNode_GridSpatialization2D::FActorCellInfo& CellInfo, TArray<UReplicationGraphNode_GridCell*>& OutNodes);

	UE_API FActorCellInfo GetCellInfoForActor(FActorRepListType Actor, const FVector& Location3D, float CullDistance);

	// This is a reused TArray for gathering actor nodes. Just to prevent using a stack based TArray everywhere or static/reset patten.
	TArray<UReplicationGraphNode_GridCell*> GatheredNodes;

	// This is a reused FActorRepListRefView for gathering actors for dormancy cleanup. Just to prevent using a stack based FActorRepListRefView or static/reset patten.
	FActorRepListRefView GatheredActors;

	friend class AReplicationGraphDebugActor;
};

// -----------------------------------


UCLASS(MinimalAPI)
class UReplicationGraphNode_AlwaysRelevant : public UReplicationGraphNode
{
	GENERATED_BODY()

public:

	UE_API UReplicationGraphNode_AlwaysRelevant();

	virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& Actor) override { }
	virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override { return false; }
	virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& Actor, bool bWarnIfNotFound=true) override { return false; }
	virtual void NotifyResetAllNetworkActors() override { }
	UE_API virtual void PrepareForReplication() override;
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;

	UE_API void AddAlwaysRelevantClass(UClass* Class);

protected:

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode> ChildNode = nullptr;

	// Explicit list of classes that are always relevant. This 
	TArray<UClass*>	AlwaysRelevantClasses;
};

// -----------------------------------

USTRUCT()
struct UE_DEPRECATED(5.2, "No longer used") FAlwaysRelevantActorInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<const UNetConnection> Connection = nullptr;

	UPROPERTY()
	TObjectPtr<AActor> LastViewer = nullptr;

	UPROPERTY()
	TObjectPtr<AActor> LastViewTarget = nullptr;

	bool operator==(UNetConnection* Other) const
	{
		return Connection == Other;
	}
};

struct FCachedAlwaysRelevantActorInfo
{
	TWeakObjectPtr<AActor> LastViewer;
	TWeakObjectPtr<AActor> LastViewTarget;
};

/** Adds actors that are always relevant for a connection. This engine version just adds the PlayerController and ViewTarget (usually the pawn) */
UCLASS(MinimalAPI)
class UReplicationGraphNode_AlwaysRelevant_ForConnection : public UReplicationGraphNode_ActorList
{
	GENERATED_BODY()

public:
	
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;

	UE_API virtual void TearDown() override;

	/** Rebuilt-every-frame list based on UNetConnection state */
	FActorRepListRefView ReplicationActorList;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** List of previously (or currently if nothing changed last tick) focused actor data per connection */
	UE_DEPRECATED(5.2, "Please use PastRelevantActorMap instead")
	UPROPERTY()
	TArray<FAlwaysRelevantActorInfo> PastRelevantActors;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	template<typename KeyType, typename ValueType>
	static void CleanupCachedRelevantActors(TMap<TObjectKey<KeyType>, ValueType>& ActorMap)
	{
		for (auto MapIt = ActorMap.CreateIterator(); MapIt; ++MapIt)
		{
			if (MapIt.Key().ResolveObjectPtr() == nullptr)
			{
				MapIt.RemoveCurrent();
			}
		}
	}

protected:
	UE_API virtual void OnCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const override;

	UE_API void AddCachedRelevantActor(const FConnectionGatherActorListParameters& Params, AActor* NewActor, TWeakObjectPtr<AActor>& LastActor);
	UE_API void UpdateCachedRelevantActor(const FConnectionGatherActorListParameters& Params, AActor* NewActor, TWeakObjectPtr<AActor>& LastActor);

	TMap<TObjectKey<UNetConnection>, FCachedAlwaysRelevantActorInfo> PastRelevantActorMap;
};

// -----------------------------------


USTRUCT()
struct FTearOffActorInfo
{
	GENERATED_BODY()
	FTearOffActorInfo() : TearOffFrameNum(0), Actor(nullptr), bHasReppedOnce(false) { }
	FTearOffActorInfo(AActor* InActor, uint32 InTearOffFrameNum) : TearOffFrameNum(InTearOffFrameNum), Actor(InActor), bHasReppedOnce(false) { }

	uint32 TearOffFrameNum;

	UPROPERTY()
	TObjectPtr<AActor> Actor;

	bool bHasReppedOnce;
};

/** Manages actors that are Tear Off. We will try to replicate these actors one last time to each connection. */
UCLASS(MinimalAPI)
class UReplicationGraphNode_TearOff_ForConnection : public UReplicationGraphNode
{
	GENERATED_BODY()

public:
	UE_API virtual void Serialize(FArchive& Ar) override;

	virtual void NotifyAddNetworkActor(const FNewReplicatedActorInfo& ActorInfo) override { }
	virtual bool NotifyRemoveNetworkActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound=true) override { return false; }
	virtual bool NotifyActorRenamed(const FRenamedReplicatedActorInfo& Actor, bool bWarnIfNotFound=true) override { return false; }
	virtual void NotifyResetAllNetworkActors() override { TearOffActors.Reset(); }
	UE_API virtual void GatherActorListsForConnection(const FConnectionGatherActorListParameters& Params) override;
	UE_API virtual void LogNode(FReplicationGraphDebugInfo& DebugInfo, const FString& NodeName) const override;
	UE_API virtual void VerifyActorReferencesInternal() override;

	UE_API void NotifyTearOffActor(AActor* Actor, uint32 FrameNum);

	// Fixme: not safe to have persistent FActorRepListrefViews yet, so need a uproperty based list to hold the persistent items.
	UPROPERTY()
	TArray<FTearOffActorInfo> TearOffActors;

	FActorRepListRefView ReplicationActorList;

protected:

	UE_API virtual void OnCollectActorRepListStats(struct FActorRepListStatCollector& StatsCollector) const override;
};

// -----------------------------------

/** Manages actor replication for an entire world / net driver */
UCLASS(MinimalAPI, transient, config=Engine)
class UReplicationGraph : public UReplicationDriver
{
	GENERATED_BODY()

public:

	UE_API UReplicationGraph();

	UE_API virtual void BeginDestroy() override;

	/** The per-connection manager class to instantiate. This will be read off the instantiated UNetReplicationManager. */
	UPROPERTY(Config)
	TSubclassOf<UNetReplicationGraphConnection> ReplicationConnectionManagerClass;

	UPROPERTY()
	TObjectPtr<UNetDriver>	NetDriver;

	/** List of connection managers. This list is not sorted and not stable. */
	UPROPERTY()
	TArray<TObjectPtr<UNetReplicationGraphConnection>> Connections;

	/** ConnectionManagers that we have created but haven't officially been added to the net driver's ClientConnection list. This is a  hack for silly order of initialization issues. */
	UPROPERTY()
	TArray<TObjectPtr<UNetReplicationGraphConnection>> PendingConnections;

	/** The max distance between an FActorDestructionInfo and a connection that we will replicate. */
	float DestructInfoMaxDistanceSquared = 15000.f * 15000.f;

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
	
	// --------------------------------------------------------------

	//~ Begin UReplicationDriver Interface
	UE_API virtual void SetRepDriverWorld(UWorld* InWorld) override;
	UE_API virtual void InitForNetDriver(UNetDriver* InNetDriver) override;
	UE_API virtual void InitializeActorsInWorld(UWorld* InWorld) override;
	virtual void ResetGameWorldState() override { }
	UE_API virtual void TearDown() override;

	/** Called by the NetDriver when the client connection is ready/added to the NetDriver's client connection list */
	UE_API virtual void AddClientConnection(UNetConnection* NetConnection) override;
	UE_API virtual void RemoveClientConnection(UNetConnection* NetConnection) override;
	UE_API virtual void AddNetworkActor(AActor* Actor) override;
	UE_API virtual void RemoveNetworkActor(AActor* Actor) override;
	UE_API virtual void ForceNetUpdate(AActor* Actor) override;
	UE_API virtual void FlushNetDormancy(AActor* Actor, bool bWasDormInitial) override;
	UE_API virtual void NotifyActorTearOff(AActor* Actor) override;
	UE_API virtual void NotifyActorFullyDormantForConnection(AActor* Actor, UNetConnection* Connection) override;
	UE_API virtual void NotifyActorDormancyChange(AActor* Actor, ENetDormancy OldDormancyState) override;
	virtual void NotifyDestructionInfoCreated(AActor* Actor, FActorDestructionInfo& DestructionInfo) override {}
	UE_API virtual void NotifyActorRenamed(AActor* Actor, UObject* PreviousOuter, FName PreviousName) override;
	UE_API virtual void SetRoleSwapOnReplicate(AActor* Actor, bool bSwapRoles) override;
	UE_API virtual bool ProcessRemoteFunction(class AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, class UObject* SubObject) override;
	UE_API virtual int32 ServerReplicateActors(float DeltaSeconds) override;
	UE_API virtual void PostTickDispatch() override;
	//~ End UReplicationDriver Interface

	UE_API virtual void RouteAddNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalInfo);

	UE_API virtual void RouteRemoveNetworkActorToNodes(const FNewReplicatedActorInfo& ActorInfo);

	UE_API bool IsConnectionReady(UNetConnection* Connection);

	UE_API void SetActorDiscoveryBudget(int32 ActorDiscoveryBudgetInKBytesPerSec);

	/** Sets the global and connection-specific cull distance setting of this actor */
	UE_API void SetAllCullDistanceSettingsForActor(const FActorRepListType& Actor, float CullDistanceSquared);

	/** Debugging facility called when invalid actor is being detected on one of many replication paths. */
	virtual void PrintGraphDebugInfo_OnInvalidActor() {}

	// --------------------------------------------------------------

	/** Creates a new node for the graph. This and UReplicationNode::CreateChildNode should be the only things that create the graph node UObjects */
	template< class T >
	T* CreateNewNode()
	{
		T* NewNode = NewObject<T>(this);
		InitNode(NewNode);
		return NewNode;
	}

	/** Creates a new node for the graph with a given base name. This and UReplicationNode::CreateChildNode should be the only things that create the graph node UObjects */
	template< class T >
	T* CreateNewNode(FName NodeBaseName)
	{
		FName UniqueNodeName = MakeUniqueObjectName(this, T::StaticClass(), NodeBaseName);
		T* NewNode = NewObject<T>(this, UniqueNodeName);
		InitNode(NewNode);
		return NewNode;
	}

	/** Add a global node to the root that will have a chance to emit actor rep lists for all connections */
	UE_API void AddGlobalGraphNode(UReplicationGraphNode* GraphNode);
	UE_API void RemoveGlobalGraphNode(UReplicationGraphNode* GraphNode);
	
	/** Associate a node to a specific connection. When this connection replicates, this GraphNode will get a chance to add. */
	UE_API void AddConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetReplicationGraphConnection* ConnectionManager);
	void AddConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetConnection* NetConnection) { AddConnectionGraphNode(GraphNode, FindOrAddConnectionManager(NetConnection)); }

	UE_API void RemoveConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetReplicationGraphConnection* ConnectionManager);
	void RemoveConnectionGraphNode(UReplicationGraphNode* GraphNode, UNetConnection* NetConnection) { RemoveConnectionGraphNode(GraphNode, FindOrAddConnectionManager(NetConnection)); }

	// --------------------------------------------------------------

	virtual UWorld* GetWorld() const override final { return GraphGlobals.IsValid() ? GraphGlobals->World : nullptr; }

	UE_API virtual void LogGraph(FReplicationGraphDebugInfo& DebugInfo) const;
	UE_API virtual void LogGlobalGraphNodes(FReplicationGraphDebugInfo& DebugInfo) const;
	UE_API virtual void LogConnectionGraphNodes(FReplicationGraphDebugInfo& DebugInfo) const;

	UE_API virtual void CollectRepListStats(struct FActorRepListStatCollector& StatCollector) const;

	UE_API void VerifyActorReferences();

	const TSharedPtr<FReplicationGraphGlobalData>& GetGraphGlobals() const { return GraphGlobals; }

	uint32 GetReplicationGraphFrame() const { return ReplicationGraphFrame; }

#if DO_ENABLE_REPGRAPH_DEBUG_ACTOR
	UE_API virtual class AReplicationGraphDebugActor* CreateDebugActor() const;
#endif

	/** Print the Net Culling Distance of all actors in the RepGraph. When a connection is specified will also output the cull distance in the connection info list*/
	UE_API void DebugPrintCullDistances(UNetReplicationGraphConnection* SpecificConnection=nullptr) const;

	/** Prioritization Constants: these affect how the final priority of an actor is calculated in the prioritize phase */
	struct FPrioritizationConstants
	{
		float MaxDistanceScaling = 60000.f * 60000.f;	// Distance scaling for prioritization scales up to this distance, everything passed this distance is the same or "capped"
		uint32 MaxFramesSinceLastRep = 20;				// Time since last rep scales up to this
		
	};
	FPrioritizationConstants PrioritizationConstants;

	struct FFastSharedPathConstants
	{
		float DistanceRequirementPct = 0.1f;	// Must be this close, as a factor of cull distance *squared*, to use fast shared replication path
		int32 MaxBitsPerFrame = 2048;			// 5kBytes/sec @ 20hz
		int32 ListSkipPerFrame = 3;
	};
	FFastSharedPathConstants FastSharedPathConstants;

	/** Invoked when a rep list is requested that exceeds the size of the preallocated lists */
	static UE_API TFunction<void(int32)> OnListRequestExceedsPooledSize;

	// --------------------------------------------------------------

	UE_API virtual void ReplicateActorsForConnection(UNetConnection* NetConnection, FPerConnectionActorInfoMap& ConnectionActorInfoMap, UNetReplicationGraphConnection* ConnectionManager, const uint32 FrameNum);

	UE_API int64 ReplicateSingleActor(const FActorRepListType& Actor, FConnectionReplicationActorInfo& ActorInfo, FGlobalActorReplicationInfo& GlobalActorInfo, FPerConnectionActorInfoMap& ConnectionActorInfoMap, UNetReplicationGraphConnection& ConnectionManager, const uint32 FrameNum);
	UE_API int64 ReplicateSingleActor_FastShared(AActor* Actor, FConnectionReplicationActorInfo& ConnectionData, FGlobalActorReplicationInfo& GlobalActorInfo, UNetReplicationGraphConnection& ConnectionManager, const uint32 FrameNum);

	UE_API void UpdateActorChannelCloseFrameNum(AActor* Actor, FConnectionReplicationActorInfo& ConnectionData, const FGlobalActorReplicationInfo& GlobalData, const uint32 FrameNum, UNetConnection* NetConnection) const;

	UE_API void NotifyConnectionSaturated(class UNetReplicationGraphConnection& Connection);

	UE_API void NotifyConnectionFastPathSaturated();

	UE_API void SetActorDestructionInfoToIgnoreDistanceCulling(AActor* DestroyedActor);

	uint16 GetReplicationPeriodFrameForFrequency(float NetUpdateFrequency) const
	{
		check(NetDriver);
		check(NetUpdateFrequency != 0.0f);

		// Replication Graph is frame based. Convert NetUpdateFrequency to ReplicationPeriodFrame based on Server MaxTickRate.
		uint32 FramesBetweenUpdates = (uint32)FMath::RoundToInt(((float)NetDriver->GetNetServerMaxTickRate()) / NetUpdateFrequency);
		FramesBetweenUpdates = FMath::Clamp<uint32>(FramesBetweenUpdates, 1, MAX_uint16);

		return (uint16)FramesBetweenUpdates;
	}

	UE_API UNetReplicationGraphConnection* FindConnectionManager(UNetConnection* NetConnection) const;

	virtual void VerifyActorReferencesInternal() {}

protected:

	UE_API virtual void InitializeForWorld(UWorld* World);

	UE_API virtual void InitNode(UReplicationGraphNode* Node);

	/** Override this function to initialize the per-class data for replication */
	UE_API virtual void InitGlobalActorClassSettings();

	/** Override this function to init/configure your project's Global Graph */
	UE_API virtual void InitGlobalGraphNodes();

	/** Override this function to init/configure graph for a specific connection. Note they do not all have to be unique: connections can share nodes (e.g, 2 nodes for 2 teams) */
	UE_API virtual void InitConnectionGraphNodes(UNetReplicationGraphConnection* ConnectionManager);

	UE_API UNetReplicationGraphConnection* FindOrAddConnectionManager(UNetConnection* NetConnection);

	UE_DEPRECATED(5.2, "This function has been replaced with a version that needs to receive a UNetReplicationGraphConnection")
	void HandleStarvedActorList(const FPrioritizedRepList& List, int32 StartIdx, FPerConnectionActorInfoMap& ConnectionActorInfoMap, uint32 FrameNum) {}

	/** Sets the next timeout frame for the actors in the list along with their dependent actors */
	UE_API void HandleStarvedActorList(const UNetReplicationGraphConnection& RepGraphConnection, const FPrioritizedRepList& List, int32 StartIdx, FPerConnectionActorInfoMap& ConnectionActorInfoMap, uint32 FrameNum);

	/** Routes a rename/outer change to every global node. Subclasses will want a more direct routing function where possible. */
	UE_API virtual void RouteRenameNetworkActorToNodes(const FRenamedReplicatedActorInfo& ActorInfo);

	/** How long, in frames, without replicating before an actor channel is closed on a connection. This is a global value added to the individual actor's ActorChannelFrameTimeout */
	uint32 GlobalActorChannelFrameNumTimeout;

	TSharedPtr<FReplicationGraphGlobalData> GraphGlobals;

	/** Temporary List we use while prioritizing actors */
	FPrioritizedRepList PrioritizedReplicationList;
	
	/** A list of nodes that can add actors to all connections. They don't necessarily *have to* add actors to each connection, but they will get a chance to. These are added via ::AddGlobalGraphNode  */
	UPROPERTY()
	TArray<TObjectPtr<UReplicationGraphNode>> GlobalGraphNodes;

	/** A list of nodes that want PrepareForReplication() to be called on them at the top of the replication frame. */
	UPROPERTY()
	TArray<TObjectPtr<UReplicationGraphNode>> PrepareForReplicationNodes;
	
	FGlobalActorReplicationInfoMap GlobalActorReplicationInfoMap;

	/** The authoritative set of "what actors are in the graph" */
	UPROPERTY()
	TSet<TObjectPtr<AActor>> ActiveNetworkActors;

	/** Special case handling of specific RPCs. Currently supports immediate send/flush for multicasts */
	TMap<FObjectKey /** UFunction* */, FRPCSendPolicyInfo> RPCSendPolicyMap;
	TClassMap<bool> RPC_Multicast_OpenChannelForClass; // Set classes to true here to open channel for them when receiving multicast RPC (and within cull distance range)

	FReplicationGraphCSVTracker CSVTracker;

	FOutBunch* FastSharedReplicationBunch = nullptr;
	class UActorChannel* FastSharedReplicationChannel = nullptr;
	FName FastSharedReplicationFuncName = NAME_None;

#if REPGRAPH_DETAILS
	bool bEnableFullActorPrioritizationDetailsAllConnections = false;
#endif

private:
	/** Whether or not a connection was saturated during an update. */
	bool bWasConnectionSaturated = false;

	/** Whether or not all fast shared replication budget has been used during replication. */
	bool bWasConnectionFastPathSaturated = false;

protected:
	/** Default Replication Path */
	UE_API void ReplicateActorListsForConnections_Default(UNetReplicationGraphConnection* ConnectionManager, FGatheredReplicationActorLists& GatheredReplicationListsForConnection, FNetViewerArray& Viewers);

	/** "FastShared" Replication Path */
	UE_API void ReplicateActorListsForConnections_FastShared(UNetReplicationGraphConnection* ConnectionManager, FGatheredReplicationActorLists& GatheredReplicationListsForConnection, FNetViewerArray& Viewers);

	/** Connections needing a FlushNet in PostTickDispatch */
	TArray<UNetConnection*> ConnectionsNeedingsPostTickDispatchFlush;

	virtual void AddReplayViewers(UNetConnection* NetConnection, FNetViewerArray& Viewers) {}

	/** Collects basic stats on the replicated actors */
	struct FFrameReplicationStats
	{
		// Total number of actors replicated.
		int32 NumReplicatedActors = 0;
        
        // Number of actors who did not send any data when ReplicateActor was called on them.
		int32 NumReplicatedCleanActors = 0;

		// Number of actors replicated using the fast path.
		int32 NumReplicatedFastPathActors = 0;

		// Total connections replicated to during the tick including children (splitscreen) and replay connections.
		int32 NumConnections = 0;

		void Reset()
		{
			*this = FFrameReplicationStats();
		}
	};
	
	/** Event called after ServerReplicateActors to dispatch the replication stats from this frame */
	UE_API virtual void PostServerReplicateStats(const FFrameReplicationStats& Stats);

private:

	UE_API UNetReplicationGraphConnection* FixGraphConnectionList(TArray<UNetReplicationGraphConnection*>& OutList, int32& ConnectionId, UNetConnection* RemovedNetConnection);

private:

	/** Collect replication data during ServerReplicateActors */
	FFrameReplicationStats FrameReplicationStats;

	/** Internal frame counter for replication. This is only updated by us. The one of UNetDriver can be updated by RPC calls and is only used to invalidate shared property CLs/serialiation data. */
	uint32 ReplicationGraphFrame = 0;

	/** Separate bandwidth cap for traffic used when opening actor channels. Ignored if set to 0 */
	int32 ActorDiscoveryMaxBitsPerFrame;

	/** Optimization to update only one connection each frame */
	int32 HeavyComputationConnectionSelector = 0;

	/** Internal time used to track when the next update should occur based on frequency settings. */
	float TimeLeftUntilUpdate = 0.f;

	UE_API UNetReplicationGraphConnection* CreateClientConnectionManagerInternal(UNetConnection* Connection);

	friend class AReplicationGraphDebugActor;

	/** Delegate to remove all network actors when a world is cleaned up. */
	FDelegateHandle WorldCleanupDelegateHandle;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

USTRUCT()
struct FLastLocationGatherInfo
{
	GENERATED_BODY()

	FLastLocationGatherInfo() : Connection(nullptr), LastLocation(ForceInitToZero), LastOutOfRangeLocationCheck(ForceInitToZero) {}
	FLastLocationGatherInfo(const UNetConnection* InConnection, FVector InLastLocation) : Connection(InConnection), LastLocation(InLastLocation), LastOutOfRangeLocationCheck(InLastLocation)  {}

	UPROPERTY()
	TObjectPtr<const UNetConnection> Connection;

	UPROPERTY()
	FVector LastLocation;

	UPROPERTY()
	FVector LastOutOfRangeLocationCheck;

	bool operator==(UNetConnection* Other) const
	{
		return Connection == Other;
	}
};

struct FRepGraphConnectionSaturationAnalytics
{
	uint32 LongestRunOfSaturatedReplications = 0;
	uint32 CurrentRunOfSaturatedReplications = 0;

	void TrackReplication(bool bIsSaturated)
	{
		if (bIsSaturated)
		{
			++CurrentRunOfSaturatedReplications;
			if (CurrentRunOfSaturatedReplications > LongestRunOfSaturatedReplications)
			{
				LongestRunOfSaturatedReplications = CurrentRunOfSaturatedReplications;
			}
		}
		else
		{
			CurrentRunOfSaturatedReplications = 0;
		}
	}
};

/** Manages actor replication for a specific connection */
UCLASS(MinimalAPI, transient)
class UNetReplicationGraphConnection : public UReplicationConnectionDriver
{
	GENERATED_BODY()

public:

	UE_API UNetReplicationGraphConnection();

	UPROPERTY()
	TObjectPtr<UNetConnection> NetConnection;

	/** A map of all of our per-actor data */
	FPerConnectionActorInfoMap ActorInfoMap;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPostReplicatePrioritizedLists, UNetReplicationGraphConnection*, FPrioritizedRepList*);
	FOnPostReplicatePrioritizedLists OnPostReplicatePrioritizeLists;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClientVisibleLevelNamesAdd, FName, UWorld*);
	FOnClientVisibleLevelNamesAdd OnClientVisibleLevelNameAdd;	// Global Delegate, will be called for every level
	TMap<FName, FOnClientVisibleLevelNamesAdd> OnClientVisibleLevelNameAddMap; // LevelName lookup map delegates

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClientVisibleLevelNamesRemove, FName);
	FOnClientVisibleLevelNamesRemove OnClientVisibleLevelNameRemove;

	// Stored list of dormant actors in a previous cell when it's been left - this is for
	// the dormant dynamic actor destruction feature.
	// This property doesn't support multiple spatialization grids and has been switched to use PrevDormantActorListPerNode.
	//FActorRepListRefView PrevDormantActorList;

#if REPGRAPH_DETAILS
	bool bEnableFullActorPrioritizationDetails = false;
#endif

	bool bEnableDebugging;

	UPROPERTY()
	TWeakObjectPtr<class AReplicationGraphDebugActor> DebugActor;

	/** Index of the connection in the global list. Will be reassigned when any client disconnects so it is a key that can be referenced only during a single tick */
	int32 ConnectionOrderNum;

	UPROPERTY()
	TArray<FLastLocationGatherInfo> LastGatherLocations;

	// Nb of bits sent for actor channel creation when a dedicated budget is allocated for this
	int32 QueuedBitsForActorDiscovery = 0;

	/** Returns connection graph nodes. This is const so that you do not mutate the array itself. You should use AddConnectionGraphNode/RemoveConnectionGraphNode.  */
	const TArray<UReplicationGraphNode*>& GetConnectionGraphNodes() const { return ConnectionGraphNodes; }

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UReplicationConnectionDriver Interface
	UE_API virtual void TearDown() override;

	UE_API virtual void NotifyClientVisibleLevelNamesAdd(FName LevelName, UWorld* StreamingWorld) override;

	virtual void NotifyClientVisibleLevelNamesRemove(FName LevelName) override { OnClientVisibleLevelNameRemove.Broadcast(LevelName); }

	UE_API virtual void NotifyActorChannelAdded(AActor* Actor, class UActorChannel* Channel) override;

	UE_API virtual void NotifyActorChannelRemoved(AActor* Actor) override;

	UE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel) override;

	UE_API virtual void NotifyAddDestructionInfo(FActorDestructionInfo* DestructInfo) override;

	UE_API virtual void NotifyRemoveDestructionInfo(FActorDestructionInfo* DestructInfo) override;

	UE_API virtual void NotifyAddDormantDestructionInfo(AActor* Actor) override;

	UE_API virtual void NotifyResetDestructionInfo() override;
	//~ End UReplicationConnectionDriver Interface

	UE_API virtual void NotifyResetAllNetworkActors();

	/** Generates a set of all the visible level names for this connection and its subconnections (if any) */
	UE_API virtual void GetClientVisibleLevelNames(TSet<FName>& OutLevelNames) const;

	/** Returns the list of visible levels cached at the start of the gather phase. */
	const TSet<FName>& GetCachedClientVisibleLevelNames() const { return CachedVisibleLevels; }

	UE_API FActorRepListRefView& GetPrevDormantActorListForNode(const UReplicationGraphNode* GridNode);

	UE_API void RemoveActorFromAllPrevDormantActorLists(AActor* InActor);
	
	struct FVisibleCellInfo
	{
		FIntPoint Location = {};
		int32 Lifetime = 0;

		// For FindByKey
		friend bool operator==(const FVisibleCellInfo& Cell, const FIntPoint& Point)
		{
			return Cell.Location == Point;
		}
	};

	UE_API TArray<FVisibleCellInfo>& GetVisibleCellsForNode(const UReplicationGraphNode* GridNode);

	UE_API void CleanupNodeCaches(const UReplicationGraphNode* Node);

	virtual void NotifyDSFNodeSaturated(const UReplicationGraphNode_DynamicSpatialFrequency* Node) {}

	UE_API virtual void TrackReplicationForAnalytics(bool bFastPathSaturated);

	UE_API virtual void ResetSaturationAnalytics();

	FRepGraphConnectionSaturationAnalytics FastPathSaturationAnalytics;
private:

	// Stored list of dormant actors in a previous cell when it's been left - this is for
	// the dormant dynamic actor destruction feature.
	TMap<TObjectKey<UReplicationGraphNode>, FActorRepListRefView> PrevDormantActorListPerNode;

	// History of visible cells per graph node used for defering dormant dynamic actors cleanup
	TMap<TObjectKey<UReplicationGraphNode>, TArray<FVisibleCellInfo>> NodesVisibleCells;
	
	friend UReplicationGraph;

	/** Holds relevant data when parsing deleted actors that could be sent to a viewer */
	struct FRepGraphDestructionViewerInfo
	{
		FRepGraphDestructionViewerInfo() = default;
		FRepGraphDestructionViewerInfo(const FVector& InViewerLocation, const FVector& InOutOfRangeLocationCheck)
			: ViewerLocation(InViewerLocation)
			, LastOutOfRangeLocationCheck(InOutOfRangeLocationCheck)
		{ }

		FVector ViewerLocation;
		FVector	LastOutOfRangeLocationCheck;
	};

	typedef TArray< FRepGraphDestructionViewerInfo, TInlineAllocator<REPGRAPH_VIEWERS_PER_CONNECTION> > FRepGraphDestructionViewerInfoArray;

	// ----------------------------------------

	/** Called right after this is created to associate with the owning Graph */
	UE_API void InitForGraph(UReplicationGraph* Graph);

	/** Called after InitForGraph is called to associate this connection manager with a net connection */
	UE_API void InitForConnection(UNetConnection* Connection);

	/** Adds a node to this connection manager */
	UE_API void AddConnectionGraphNode(UReplicationGraphNode* Node);

	/** Remove a node to this connection manager */
	UE_API void RemoveConnectionGraphNode(UReplicationGraphNode* Node);

	UE_API bool PrepareForReplication();

	UE_API int64 ReplicateDestructionInfos(const FRepGraphDestructionViewerInfoArray& DestructionViewersInfo, const FReplicationGraphDestructionSettings& DestructionSettings);
	
	UE_API int64 ReplicateDormantDestructionInfos();

	/** Update the last location for viewers on a connection */
	UE_API void UpdateGatherLocationsForConnection(const FNetViewerArray& ConnectionViewers, const FReplicationGraphDestructionSettings& DestructionSettings);

    /** Update the location info of a specific viewer */
	UE_API void OnUpdateViewerLocation(FLastLocationGatherInfo* LocationInfo, const FNetViewer& Viewer, const FReplicationGraphDestructionSettings& DestructionSettings);

	UE_API void SetActorNotDormantOnConnection(AActor* InActor);

	UE_API void BuildVisibleLevels();

	UPROPERTY()
	TArray<TObjectPtr<UReplicationGraphNode>> ConnectionGraphNodes;

	UPROPERTY()
	TObjectPtr<UReplicationGraphNode_TearOff_ForConnection> TearOffNode;
	
	/** DestructionInfo handling. This is how we send "delete this actor" to clients when the actor is deleted on the server (placed in map actors) */
	struct FCachedDestructInfo
	{
		FCachedDestructInfo(FActorDestructionInfo* InDestructInfo) : DestructionInfo(InDestructInfo), CachedPosition(InDestructInfo->DestroyedPosition) {}
		bool operator==(const FCachedDestructInfo& rhs) const { return DestructionInfo == rhs.DestructionInfo; };
		bool operator==(const FActorDestructionInfo* InDestructInfo) const { return InDestructInfo == DestructionInfo; };
		
		FActorDestructionInfo* DestructionInfo;
		FVector CachedPosition;

		void CountBytes(FArchive& Ar) const
		{
			if (DestructionInfo)
			{
				Ar.CountBytes(sizeof(FActorDestructionInfo), sizeof(FActorDestructionInfo));
				DestructionInfo->CountBytes(Ar);
			}
		}
	};

	/** 
	* List of destroyed actors that were too far from the connection to be relevant.
	* Is periodically evaluated when the viewer crosses a specific distance.
	*/
	TArray<FCachedDestructInfo> OutOfRangeDestroyedActors;

	/** List of destroyed actors that need to be replicated */
	TArray<FCachedDestructInfo> PendingDestructInfoList;

	/** Set used to guard against double adds into PendingDestructInfoList */
	TSet<FActorDestructionInfo*> TrackedDestructionInfoPtrs;

	struct FCachedDormantDestructInfo
	{
		TWeakObjectPtr<ULevel> Level;
		TWeakObjectPtr<UObject> ObjOuter;
		FNetworkGUID NetGUID;
		FString PathName;
	};

	/** List of dormant actors that should be removed from the client */
	TArray<FCachedDormantDestructInfo> PendingDormantDestructList;

	/** Set used to guard against double adds into PendingDormantDestructList */
	TSet<FNetworkGUID> TrackedDormantDestructionInfos;

	/** List of level names visible to this connection. Valid during the Gather phase */
	TSet<FName> CachedVisibleLevels;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------


/** Specialized actor for replicating debug information about replication to specific connections. This actor is never spawned in shipping builds and never counts towards bandwidth limits */
UCLASS(MinimalAPI, NotPlaceable, Transient)
class AReplicationGraphDebugActor : public AActor
{
	GENERATED_BODY()

public:

	AReplicationGraphDebugActor()
	{
		bReplicates = true; // must be set for RPCs to be sent
		SetNetUpdateFrequency(10.0f);
	}

	// To prevent demo netdriver from replicating.
	virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override { return false; }
	virtual bool IsReplayRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation, const float CullDistanceSquared) const override { return false; }

	UPROPERTY()
	TObjectPtr<UReplicationGraph> ReplicationGraph;

	UFUNCTION(Server, Reliable)
	UE_API void ServerStartDebugging();

	UFUNCTION(Server, Reliable)
	UE_API void ServerStopDebugging();

	UFUNCTION(Server, Reliable)
	UE_API void ServerCellInfo();

	UFUNCTION(Server, Reliable)
	UE_API void ServerPrintAllActorInfo(const FString& Str);

	UFUNCTION(Server, Reliable)
	UE_API void ServerSetCullDistanceForClass(UClass* Class, float CullDistance);

	UFUNCTION(Server, Reliable)
	UE_API void ServerSetPeriodFrameForClass(UClass* Class, int32 PeriodFrame);

	UFUNCTION(Server, Reliable)
	UE_API void ServerSetConditionalActorBreakpoint(AActor* Actor);

	UFUNCTION(Server, Reliable)
	UE_API void ServerPrintCullDistances();

	UFUNCTION(Client, Reliable)
	UE_API void ClientCellInfo(FVector CellLocation, FVector CellExtent, const TArray<AActor*>& Actors);

	UE_API void PrintCullDistances();

	UE_API void PrintAllActorInfo(FString MatchString);


	UPROPERTY()
	TObjectPtr<UNetReplicationGraphConnection> ConnectionManager;

	UE_API virtual UNetConnection* GetNetConnection() const override;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
struct FReplicationGraphDestructionSettings
{
	FReplicationGraphDestructionSettings(float InDestructInfoMaxDistanceSquared, float InOutOfRangeDistanceCheckThresholdSquared)
		: DestructInfoMaxDistanceSquared(InDestructInfoMaxDistanceSquared)
		, OutOfRangeDistanceCheckThresholdSquared(InOutOfRangeDistanceCheckThresholdSquared)
		, MaxPendingListDistanceSquared(DestructInfoMaxDistanceSquared + OutOfRangeDistanceCheckThresholdSquared)
	{ }

	float DestructInfoMaxDistanceSquared;
	float OutOfRangeDistanceCheckThresholdSquared;
	float MaxPendingListDistanceSquared; 
};

#undef UE_API
