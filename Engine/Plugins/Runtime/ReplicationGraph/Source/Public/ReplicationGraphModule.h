// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/** Module definition for replication graph */
class FReplicationGraphModule : public FDefaultModuleImpl
{
public:

private:
	
};
