// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "GameFramework/WorldSettings.h"
#include "Net/DataBunch.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Templates/Greater.h"
#include "Templates/Requires.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"

#include <type_traits>

#include "ReplicationGraphTypes.generated.h"

#define UE_API REPLICATIONGRAPH_API

class AActor;
class AController;
class UActorChannel;
class UNetConnection;
class UNetReplicationGraphConnection;
class UReplicationGraph;
class UReplicationGraphNode;

struct FActorDestructionInfo;
struct FReplicationGraphDebugInfo;
struct FConnectionGatherActorListParameters;

REPLICATIONGRAPH_API DECLARE_LOG_CATEGORY_EXTERN( LogReplicationGraph, Log, All );

LLM_DECLARE_TAG_API(NetRepGraph, REPLICATIONGRAPH_API);

// Check aliases for within the system. The intention is that these can be flipped to checkSlow once the system is stable.

// Macro enable extra logging/bookkeeping. Disabled in Test/Shipping for perf.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define REPGRAPH_DETAILS 1
	#define DO_REPGRAPH_DETAILS(X) (X)
	#define repCheck(x) check(x)
	#define repCheckf(expr, format, ...) checkf(expr, format, ##__VA_ARGS__ )
	#define RG_QUICK_SCOPE_CYCLE_COUNTER(x) QUICK_SCOPE_CYCLE_COUNTER(x)
	REPLICATIONGRAPH_API extern int32 CVar_RepGraph_Verify;
#else 
	#define REPGRAPH_DETAILS 0
	#define DO_REPGRAPH_DETAILS(X) 0
	#define repCheck(x)
	#define repCheckf(expr, format, ...)
	#define RG_QUICK_SCOPE_CYCLE_COUNTER(X)
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)	
#define REPGRAPH_STR(x) #x
#define REPGRAPH_DEVCVAR_SHIPCONST(Type,VarName,Var,Value,Help) \
	Type Var = Value; \
	static FAutoConsoleVariableRef Var##CVar(TEXT(VarName), Var, TEXT(Help), ECVF_Cheat );
#else
#define REPGRAPH_DEVCVAR_SHIPCONST(Type,VarName,Var,Value,Help) \
	const Type Var = Value;
#endif

#ifndef REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE
#define REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE 1
#endif // REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Actor Replication List Types
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

// Currently we are using raw AActor* for our replication lists. We may want to change this one day to be an ID or something else that can index
// fast into arrays etc. (Currently we are using TMaps for associative data, static arrays would be faster but introduce constraints and headaches)
// So for now, using a typedef and some helper functions to call out the interface/usage of FActorRepListType.

// This define embeds an extra FObjectKey for a validity check
#ifndef UE_ENABLE_ACTOR_REPLIST_TRACKING
	#define UE_ENABLE_ACTOR_REPLIST_TRACKING 0
#endif

#define UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY			(UE_SERVER && UE_ENABLE_ACTOR_REPLIST_TRACKING)

#if !UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY
typedef AActor* FActorRepListType;
#else // UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY

namespace UE::Net::RepGraph
{
	// Whether to store weak pointers in actor replication lists and validate them on use.
	// When false, raw pointers are used instead.
	inline bool bUseWeakPointers = false;
}

struct FActorRepListType
{
private:
	// Actor pointer (raw or weak).
	//
	// Most accesses should be via GetActor() or SetActor().
	//
	// Type is set during construction depending on bUseWeakPointers: FWeakObjectPtr when true, AActor* when false. Most functions should depend on
	// the actual type, only SetActor() depends on bUseWeakPointers so the configuration can change at runtime without breaking extant instances.
	//
	// Could be condensed to sizeof(AActor*) aka sizeof(FWeakObjectPtr) by storing the type flag in the high bit and masking it out.
	TVariant<AActor*, FWeakObjectPtr> ActorUnion;

public:
	/** More info for debugging - ActorRaw's name */
	FName ActorName;
	/** More info for debugging - ActorRaw's Owner name */
	FName OwnerName;
	/** More info for debugging - ActorRaw's Outer's package name */
	FName OuterPackageName;

	FActorRepListType()
	{
		SetActor(nullptr);
	}

	// Permits implicit construction so FActorRepListType can be a drop-in substitute for `AActor*` depending on an ifdef.
	/*implicit*/ FActorRepListType(AActor* InActor)
	{
		SetActor(InActor);
	}

	// Copies/moves are clones that don't need to validate the object.
	FActorRepListType(const FActorRepListType& InActor) = default;
	FActorRepListType(FActorRepListType&& InActor) = default;
	FActorRepListType& operator=(const FActorRepListType& InActor) = default;
	FActorRepListType& operator=(FActorRepListType&& InActor) = default;

	// to support conversion from TObjectPtr<ASubclassOfActor>
	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<const T&, AActor*>)
	>
	FActorRepListType(const T& InActor)
	{
		SetActor(InActor);
	}

	operator AActor* () const { return GetActor(); }
	AActor* operator->() const { return GetActor(); }
	explicit operator uint64() const { return GetTypeHash(); }
	FActorRepListType& operator=(AActor* InActor)
	{
		SetActor(InActor);
		return *this;
	}
	FActorRepListType& operator=(TObjectPtr<AActor> InActor)
	{
		SetActor(InActor.Get());
		return *this;
	}

	friend bool operator==(const FActorRepListType& Left, const FActorRepListType& Right)
	{
		// Avoid FWeakObjectPtr::operator==() because it resolves both weak pointers if they don't match.
		if (const FWeakObjectPtr* LeftPtr = Left.ActorUnion.TryGet<FWeakObjectPtr>(), *RightPtr = Right.ActorUnion.TryGet<FWeakObjectPtr>();
			LeftPtr && RightPtr)
		{
			return LeftPtr->HasSameIndexAndSerialNumber(*RightPtr);
		}

		// Differing types or both raw pointers; weak pointer must be resolved.
		return Left.GetActor() == Right.GetActor();
	}
	friend bool operator!=(const FActorRepListType& Left, const FActorRepListType& Right)
	{
		return !(Left == Right);
	}

	friend bool operator==(const FActorRepListType& RepListActor, AActor* RawActor)
	{
		return RepListActor.GetActor() == RawActor;
	}
	friend bool operator!=(const FActorRepListType& RepListActor, AActor* RawActor)
	{
		return !(RepListActor == RawActor);
	}

	friend bool operator==(AActor* RawActor, const FActorRepListType& RepListActor)
	{
		return RawActor == RepListActor.GetActor();
	}
	friend bool operator!=(AActor* RawActor, const FActorRepListType& RepListActor)
	{
		return !(RepListActor == RawActor);
	}

	// comparison with nullptr without resolving weak pointer or implicitly constructing a new FActorRepListType(nullptr).
	friend bool operator==(const FActorRepListType& RepListActor, TYPE_OF_NULLPTR)
	{
		if (const FWeakObjectPtr* WeakPtrPtr = RepListActor.ActorUnion.TryGet<FWeakObjectPtr>(); WeakPtrPtr)
		{
			return WeakPtrPtr->IsExplicitlyNull();
		}

		AActor* const* RawActorPtr = RepListActor.ActorUnion.TryGet<AActor*>();
		return ensure(RawActorPtr) && *RawActorPtr == nullptr;
	}
	friend bool operator!=(const FActorRepListType& RepListActor, TYPE_OF_NULLPTR)
	{
		return !(RepListActor == nullptr);
	}
	friend bool operator==(TYPE_OF_NULLPTR, const FActorRepListType& RepListActor)
	{
		return RepListActor == nullptr;
	}
	friend bool operator!=(TYPE_OF_NULLPTR, const FActorRepListType& RepListActor)
	{
		return !(RepListActor == nullptr);
	}

	bool IsValid() const
	{
		if (ActorUnion.IsType<AActor*>())
		{
			return ActorUnion.Get<AActor*>() != nullptr;
		}

		return ActorUnion.Get<FWeakObjectPtr>().IsValid();
	}

	inline AActor* GetActor() const
	{
		if (ActorUnion.IsType<AActor*>())
		{
			return ActorUnion.Get<AActor*>();
		}

		const FWeakObjectPtr& WeakActor = ActorUnion.Get<FWeakObjectPtr>();

		if (WeakActor.IsExplicitlyNull())
		{
			return nullptr;
		}

		// Permit pointers to objects marked for destruction but not destroyed yet.
		constexpr bool bPermitGarbage = true;
		AActor* Actor = Cast<AActor>(WeakActor.Get(bPermitGarbage));
		ensureMsgf(
			Actor,
			TEXT("RepGraph contains a destroyed actor (already gone): Name=%s Owner=%s Package=%s"),
			*ActorName.ToString(),
			*OwnerName.ToString(),
			*OuterPackageName.ToString());

		return Actor;
	}

	// Get hash. Prefer this be a member function than standalone to avoid implicit conversions to FActorRepListType.
	uint32 GetTypeHash() const
	{
		// Must resolve the pointer to ensure hashing produces identical results regardless of whether the stored pointer is weak or raw.
		// (Hashing FWeakObjectPtr::ObjectIndex for weak or UObjectBase::InternalIndex for raw would be OK, but both are inaccessible.)
		return ::GetTypeHash(GetActor());
	}

private:
	void SetActor(AActor* Actor)
	{
		if (UE::Net::RepGraph::bUseWeakPointers)
		{
			ActorUnion.Emplace<FWeakObjectPtr>(Actor);
		}
		else
		{
			ActorUnion.Emplace<AActor*>(Actor);
		}

		SetDebugInfo();
	}

	void SetDebugInfo()
	{
		ActorName = OwnerName = OuterPackageName = NAME_None;
		if (AActor* Actor = GetActor(); LIKELY(Actor))
		{
			ActorName = Actor->GetFName();
			OwnerName = Actor->GetOwner() ? Actor->GetOwner()->GetFName() : NAME_None;
			if (LIKELY(Actor->GetOuter()))
			{
				// judging by the implementation GetPackage() cannot return nullptr, but play it safe
				UPackage* Pkg = Actor->GetOuter()->GetPackage();
				if (LIKELY(Pkg))
				{
					OuterPackageName = Pkg->GetFName();
				}
			}
		}
	}
};

template< class T > inline T* Cast(const FActorRepListType& Src) { return Cast<T>(Src.GetActor()); }
template< class T > inline T* ExactCast(const FActorRepListType& Src) { return ExactCast<T>(Src.GetActor()); }
template< class T > inline T* CastChecked(const FActorRepListType& Src, ECastCheckedType::Type CheckType = ECastCheckedType::NullChecked)
{
	return CastChecked<T>(Src.GetActor(), CheckType);
}
#endif // UE_ACTOR_REPLIST_TYPE_EXTRA_SAFETY

inline FString GetActorRepListTypeDebugString(const FActorRepListType& In) { return GetNameSafe(In); }
inline UClass* GetActorRepListTypeClass(const FActorRepListType& In) { return In->GetClass(); }

// Generic flags that describe an actor list. Currently only used for "Default" vs "FastShared" path. These are for unsorted lists that are returned from the graph, merged together, sorted, and finally replicated until bandwidth limit is hit
enum class EActorRepListTypeFlags : uint8
{
	Default = 0,
	FastShared = 1,
	Max, // Always keep last
};

// Tests if an actor is valid for replication: not pending kill, etc. Says nothing about wanting to replicate or should replicate, etc.
inline bool IsActorValidForReplication(const AActor* Actor)
{
	return Actor && !Actor->IsActorBeingDestroyed() && IsValidChecked(Actor) && !Actor->IsUnreachable(); 
}
inline bool IsActorValidForReplication(const FActorRepListType& In)
{
	return IsActorValidForReplication(static_cast<const AActor*>(In));
}

REPLICATIONGRAPH_API void LogMoreInfoOnIsActorValidFailure(const FActorRepListType& In);
inline bool IsActorValidForReplication_LogMoreInfo(const FActorRepListType& In)
{ 
	if (LIKELY(IsActorValidForReplication(In)))
	{
		return true;
	}

	LogMoreInfoOnIsActorValidFailure(In);
	return false;
}

// Tests if an actor is valid for replication gathering. Meaning, it can be gathered from the replication graph and considered for replication.
inline bool IsActorValidForReplicationGather(const FActorRepListType& In)
{
	const AActor* Actor = In;

	if (!Actor)
		return false;

	if (!IsActorValidForReplication(Actor))
		return false;

	if (Actor->GetIsReplicated() == false)
		return false;

	if (Actor->GetTearOff())
		return false;

	if (Actor->NetDormancy == DORM_Initial && Actor->IsNetStartupActor())
		return false;

/*
	These checks were done in legacy code and we would like to avoid them.

	// Actors should finish initialization outside of the replication loop. Maybe some weird multi frame delayed case?
	if (!Actor->IsActorInitialized())
		return false;

	// This check is slow and is not needed unless you are streaming levels on the server. If needed this should be opt in globally some how.
	ULevel* Level = In->GetLevel();
	if ( Level->HasVisibilityChangeRequestPending() || Level->bIsAssociatingLevel )
		return false;

*/

	return true;
}

/** The actual underlying list type that the system uses for a list of replicated actors. These are never manually allocated or even really used. All Replication code interacts with the "View" types below  */
struct FActorRepList : FNoncopyable
{
	/** List Header */
	int32 RefCount;
	int32 Max;
	int32 Num;

	/** the "used bit" from the block we came from, so that we can clear it fast */
	FBitReference UsedBitRef;

	/** Variable length Data segment */
	FActorRepListType Data[];

	/** For TRefCountPtr usage */
	void AddRef() { RefCount++; }
	UE_API void Release();

	void CountBytes(FArchive& Ar) const
	{
		Ar.CountBytes(Num * sizeof(FActorRepListType), Max * sizeof(FActorRepListType));
	}
};

/** This is a base templated type for the list "Views". This provides basic read only access between the two real views. */
template<typename PointerType>
struct TActorRepListViewBase
{
	TActorRepListViewBase() { }
	TActorRepListViewBase(const PointerType& In) : RepList(In) { }

	inline int32 Num() const { return RepList->Num; }
	inline const FActorRepListType& operator[](int32 idx) const  { repCheck(RepList); repCheck(RepList->Max > idx); return RepList->Data[idx]; }

	/** Resets the view to null - meaning it is not pointing to any list (as opposed to an empty list of some preallocated size) */
	inline void ResetToNull() { RepList = nullptr; }
	inline bool IsValid() const { return RepList != nullptr; }

	int32 IndexOf(const FActorRepListType& Value) const
	{
		repCheck(RepList);
		FActorRepListType* Data = RepList->Data;
		int32 Num = RepList->Num;
		for (int32 idx=0 ; idx < Num; ++idx)
		{
			if (Data[idx] == Value)
			{
				return idx;
			}
		}
		return -1;
	}

	bool Contains(const FActorRepListType& Value) const { return IndexOf(Value) != -1; }

	/** Add contents to TArray. this is intended for debugging/ease of use */
	void AppendToTArray(TArray<FActorRepListType>& OutArray) const
	{
		if (IsValid())
		{
			for (FActorRepListType Actor : *this)
			{
				OutArray.Add(Actor);
			}
		}
	}

	void AppendToTSet(TSet<FActorRepListType>& OutSet) const
	{
		if (IsValid())
		{
			for (FActorRepListType Actor : *this)
			{
				OutSet.Add(Actor);
			}
		}
	}
	
	PointerType RepList;

	FString BuildDebugString() const
	{
		FString Str;
		if (Num() > 0)
		{
			Str += GetActorRepListTypeDebugString(RepList->Data[0]);
			for (int32 i=1; i < Num(); ++i)
			{
				Str += TEXT(", ") + GetActorRepListTypeDebugString(RepList->Data[i]);
			}
		}
		return Str;
	}

	FActorRepListType* begin() const { return RepList->Data; }
	FActorRepListType* end() const { return RepList->Data + RepList->Num; }

private:

	inline friend FActorRepListType* begin(const TActorRepListViewBase<PointerType>& View) { return View.RepList->Data; }
	inline friend FActorRepListType* end(const TActorRepListViewBase<PointerType>& View) { return View.RepList->Data + View.RepList->Num; }
};

/**
 * Holds a list of replicated actors that can be added/removed to.
 */
struct FActorRepListRefView
{
	FActorRepListRefView()
	{
	}

	/** Empties the array but does not deallocate the internal memory. Will be resized if the specified max size is bigger than the current max. */
	void Reset(int32 ExpectedMaxSize=0)
	{
		RepList.Reset(ExpectedMaxSize);
	}

	/** Preallocate the array so it can hold the specified size */
	void Reserve(int32 Size)
	{
		RepList.Reserve(Size);
	}

	bool ConditionalAdd(const FActorRepListType& NewElement)
	{
		if (IsActorValidForReplicationGather(NewElement))
		{
			Add(NewElement);
			return true;
		}
		return false;
	}

	void Add(const FActorRepListType& NewElement)
	{
		RepList.Add(NewElement);
	}

	/** Removes the element quickly but changes the list order */
	bool RemoveFast(const FActorRepListType& ElementToRemove, EAllowShrinking AllowShrink = EAllowShrinking::Default)
	{
		return RepList.RemoveSingleSwap(ElementToRemove, AllowShrink) > 0;
	}

	/** Removes the element but keeps the order intact. Generally not recommended for large lists. */
	bool RemoveSlow(const FActorRepListType& ElementToRemove)
	{
		return RepList.RemoveSingle(ElementToRemove) > 0;
	}

	void RemoveAtSwap(int32 idx)
	{
		RepList.RemoveAtSwap(idx);
	}

	void CopyContentsFrom(const FActorRepListRefView& Source)
	{
		RepList = Source.RepList;
	}

	void AppendContentsFrom(const FActorRepListRefView& Source)
	{
		RepList.Append(Source.RepList);
	}

	UE_API bool VerifyContents_Slow() const;

	/** Add contents to TArray/TSet. this is intended for debugging/ease of use */
	void AppendToTArray(TArray<FActorRepListType>& OutArray) const
	{
		OutArray.Append(RepList);
	}
	void AppendToTSet(TSet<FActorRepListType>& OutSet) const
	{
		OutSet.Append(RepList);
	}

	UE_API FString BuildDebugString() const;

	/**
	 * Base view functions.
	 */

	FActorRepListType& operator[](int32 idx)				{ return RepList[idx]; }
	const FActorRepListType& operator[](int32 idx) const	{ return RepList[idx]; }

	TArray<FActorRepListType>::RangedForIteratorType begin()			{ return RepList.begin(); }
	TArray<FActorRepListType>::RangedForConstIteratorType begin() const { return RepList.begin(); }
	TArray<FActorRepListType>::RangedForIteratorType end()				{ return RepList.end(); }
	TArray<FActorRepListType>::RangedForConstIteratorType end() const	{ return RepList.end(); }

	bool IsEmpty() const
	{
		return RepList.Num() <= 0;
	}

	int32 Num() const
	{ 
		return RepList.Num(); 
	}

	/** Resets the container and returns the memory it held */
	void TearDown()
	{
		RepList.Empty();
	}
	
	int32 IndexOf(const FActorRepListType& Value) const
	{
		return RepList.IndexOfByKey(Value);
	}

	bool Contains(const FActorRepListType& Value) const
	{
		return RepList.Contains(Value);
	}

	void CountBytes(FArchive& Ar) const
	{
		RepList.CountBytes(Ar);
	}

private:

	friend struct FActorRepListStatCollector;

	TArray<FActorRepListType> RepList;
};

/**
 * Gives temporary read-only access to a FActorRepListRefView by holding a reference to it.
 */
struct
REPLICATIONGRAPH_API 
UE_DEPRECATED(5.5, "Use TArrayView<const FActorRepListType> instead")
FActorRepListConstView
{
	explicit FActorRepListConstView(const FActorRepListRefView& InListReferenced) :
		ListReferenced(InListReferenced)
	{}

	FActorRepListType operator[](int32 idx) const
	{
		return ListReferenced[idx];
	}

	int32 Num() const
	{
		return ListReferenced.Num();
	}

	TArray<FActorRepListType>::RangedForConstIteratorType begin() const { return ListReferenced.begin(); }
	TArray<FActorRepListType>::RangedForConstIteratorType end() const { return ListReferenced.end(); }

private:
	const FActorRepListRefView& ListReferenced;
};

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Intended to be called from immediate mode window while debugging
	extern "C" DLLEXPORT void PrintRepListDetails(int32 PoolSize, int32 BlockIdx, int32 ListIdx);
	extern "C" DLLEXPORT void PrintRepListStats(int32 mode=0);
	
	void PrintRepListStatsAr(int32 mode, FOutputDevice& Ar=*GLog);	
#endif


// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Gathering Parameters
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

// This represents "the list of gathered actors". This is what we push down the Replication Graph and nodes will either Push/Pop List Categories or will add their Replication Lists.
struct FGatheredReplicationActorLists
{
	void AddReplicationActorList(const FActorRepListRefView& List, EActorRepListTypeFlags Flags = EActorRepListTypeFlags::Default)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVar_RepGraph_Verify)
			List.VerifyContents_Slow();
#endif
		List.AppendToTArray(ReplicationLists[(uint32)Flags]);
	}

	inline void Reset()
	{
		for (uint32 i = (uint32)EActorRepListTypeFlags::Default; i < (uint32)EActorRepListTypeFlags::Max; ++i)
		{
			ReplicationLists[i].Reset();
		}
	}
	inline int32 NumLists() const
	{
		return ReplicationLists.Num();
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Use ViewActors() instead")
	inline const TArray<FActorRepListConstView>& GetLists(EActorRepListTypeFlags ListFlags) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	inline TArrayView<const FActorRepListType> ViewActors(EActorRepListTypeFlags ListFlags) const
	{
		return ReplicationLists[(uint32)ListFlags];
	}

	inline bool ContainsLists(EActorRepListTypeFlags Flags) const
	{
		return ReplicationLists[(uint32)Flags].Num() > 0;
	}

private:

	TStaticArray< TArray<FActorRepListType>, (uint32)EActorRepListTypeFlags::Max > ReplicationLists;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// NewReplicatedActorInfo
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/** This is the struct we use to push new replication actors into the graph. "New" doesn't mean "newly spawned" it means "new to the graph". FIXME: Please suggest a better name! */
struct FNewReplicatedActorInfo
{
	explicit FNewReplicatedActorInfo(const FActorRepListType& InActor) : Actor(InActor), Class(InActor->GetClass())
	{
		StreamingLevelName = GetStreamingLevelNameOfActor(Actor);
	}

	explicit FNewReplicatedActorInfo(const FActorRepListType& InActor, FName OverrideLevelName)
		: Actor(InActor)
		, StreamingLevelName(OverrideLevelName)
		, Class(InActor->GetClass())
	{
	}

	AActor* GetActor() const { return Actor; }

	REPLICATIONGRAPH_API static FName GetStreamingLevelNameOfActor(const AActor* Actor);

	FActorRepListType Actor;
	FName StreamingLevelName;
	UClass* Class;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// RenamedReplicatedActorInfo
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/** Used to update renamed (changed outer/level) actors in the graph. */
struct FRenamedReplicatedActorInfo
{
	explicit FRenamedReplicatedActorInfo(const FActorRepListType& InActor, FName InPreviousStreamingLevelName)
		: NewActorInfo(InActor)
		, OldActorInfo(InActor, InPreviousStreamingLevelName)
	{
	}

	/** Info that stores the actor's new level name */
	FNewReplicatedActorInfo NewActorInfo;

	/** Info that stores the actor's old level name */
	FNewReplicatedActorInfo OldActorInfo;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Level based actor lists 
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/**
* Holds a list of actors grouped by their streaming level
*/
struct FStreamingLevelActorListCollection
{
	UE_API void AddActor(const FNewReplicatedActorInfo& ActorInfo);

	/** 
	* Removes the actor but keep the actor order list intact 
	* 
	* @param ActorInfo  The actor to be removed
	* @param bWarnIfNotFound When true we log a warning when the actor was not in any list
	* @param Outer Optional pointer to the owner of the list if you want to output the owner name in any log messages
	*/
	UE_API bool RemoveActor(const FNewReplicatedActorInfo& ActorInfo, bool bWarnIfNotFound, UObject* Outer=nullptr);

	/**
	* Removes the actor but using the faster swap operation that change the ordering of the list
	*
	* @param ActorInfo  The actor to be removed
	* @param Outer Optional pointer to the owner of the list if you want to output the owner name in any log messages
	*/
	UE_API bool RemoveActorFast(const FNewReplicatedActorInfo& ActorInfo, UObject* Outer=nullptr);

	/**
 	* Attempts to remove the actor from the level explicitly provided. Can be used to update the list
	* if the actor's level changes (by providing the actor's previous level).
	*/
	UE_API bool RemoveActorFromLevelFast(AActor* Actor, FName LevelName);

	UE_API void Reset();

	/** Add to the Params' gather list the list of actors for levels that are visible to that connection */
	UE_API void Gather(const FConnectionGatherActorListParameters& Params) const;

	/** Add to the passed gather list the list of actors that are visible to the connection */
	UE_API void Gather(const UNetReplicationGraphConnection& ConnectionManager, FGatheredReplicationActorLists& OutGatheredList) const;

	/** Add to the passed gather list all list of actors held */
	UE_API void AppendAllLists(FGatheredReplicationActorLists& OutGatheredList) const;

	/** Is this actor held in one of the actor lists */
	UE_API bool Contains(const FNewReplicatedActorInfo& ActorInfo) const;

	UE_API void DeepCopyFrom(const FStreamingLevelActorListCollection& Source);
	UE_API void GetAll_Debug(TArray<FActorRepListType>& OutArray) const;
	UE_API void Log(FReplicationGraphDebugInfo& DebugInfo) const;
	
	int32 NumLevels() const { return StreamingLevelLists.Num(); }
	
	UE_API void TearDown();

	struct FStreamingLevelActors
	{
		FStreamingLevelActors(FName InName) : StreamingLevelName(InName)  { repCheck(InName != NAME_None); }

		FName StreamingLevelName;
		FActorRepListRefView ReplicationActorList;
		bool operator==(const FName& InName) const { return InName == StreamingLevelName; };
	};

	/** Lists for streaming levels. Actors that "came from" streaming levels go here. These lists are only returned if the connection has their streaming level loaded. */
	static const int32 NumInlineAllocations = 4;
	TArray<FStreamingLevelActors, TInlineAllocator<NumInlineAllocations>> StreamingLevelLists;

	void CountBytes(FArchive& Ar) const
	{
		StreamingLevelLists.CountBytes(Ar);

		for (const FStreamingLevelActors& List : StreamingLevelLists)
		{
			List.ReplicationActorList.CountBytes(Ar);
		}
	}
};


/**
* Holds a list of actors grouped by the permanent level or their streaming level
*/
struct FLevelBasedActorList
{
	UE_API void AddNetworkActor(AActor* NetActor);
	UE_API bool RemoveNetworkActor(const FNewReplicatedActorInfo& NetActor);
	bool RemoveNetworkActor(AActor* NetActor)
	{
		FNewReplicatedActorInfo ActorInfo(NetActor);
		return RemoveNetworkActor(ActorInfo);
	}

	UE_API bool RemoveNetworkActorOrdered(AActor* NetActor);

	UE_API void Reset();

	UE_API bool Contains(AActor* NetActor) const;

	/** Gather the list of actors visible to this connection and add them to the gather list */
	UE_API void Gather(const FConnectionGatherActorListParameters& Params) const;
	UE_API void Gather(const UNetReplicationGraphConnection& ConnectionManager, FGatheredReplicationActorLists& OutGatheredList) const;

	/** Add a reference to all the lists held here to the gather list */
	UE_API void AppendAllLists(FGatheredReplicationActorLists& OutGatheredList) const;

	/** Returns the list of all actors held by the permenant and streaming level list. Prefer using AppendAllLists to prevent copying TArrays for nothing. */
	UE_API void GetAllActors(TArray<AActor*>& OutAllActors) const;

	UE_API void CountBytes(FArchive& Ar) const;

	/** Update actor to new level */
	UE_API void UpdateActorLevel(AActor* NetActor, FName PreviousLevelName);

private:

	FActorRepListRefView PermanentLevelActors;

	FStreamingLevelActorListCollection StreamingLevelActors;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Per Class/Actor Global/PerConnectino Data Structs
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/** Per-Class actor data about how the actor replicates */
USTRUCT()
struct FClassReplicationInfo
{
	GENERATED_BODY()

	UPROPERTY()
	float DistancePriorityScale = 1.f;
	UPROPERTY()
	float StarvationPriorityScale = 1.f;
	UPROPERTY()
	float AccumulatedNetPriorityBias = 0.f;
	
	UPROPERTY()
	uint16 ReplicationPeriodFrame = 1;
	UPROPERTY()
	uint16 FastPath_ReplicationPeriodFrame = 1;
	UPROPERTY()
	uint16 ActorChannelFrameTimeout = 4;

	TFunction<bool(AActor*)> FastSharedReplicationFunc = nullptr;
	FName FastSharedReplicationFuncName = NAME_None;

	void SetCullDistanceSquared(float InCullDistanceSquared)
	{
		CullDistanceSquared = InCullDistanceSquared;
		CullDistance = FMath::Sqrt(CullDistanceSquared);
	}

	float GetCullDistance() const { return CullDistance; }
	float GetCullDistanceSquared() const { return CullDistanceSquared; }

	FString BuildDebugStringDelta() const
	{
		FClassReplicationInfo DefaultValues;
		FString Str;
		if (CullDistance != DefaultValues.CullDistance)
		{
			Str += FString::Printf(TEXT("CullDistance: %.2f "), CullDistance);
		}
		if (StarvationPriorityScale != DefaultValues.StarvationPriorityScale)
		{
			Str += FString::Printf(TEXT("StarvationPriorityScale: %.2f "), StarvationPriorityScale);
		}
		if (DistancePriorityScale != DefaultValues.DistancePriorityScale)
		{
			Str += FString::Printf(TEXT("DistancePriorityScale: %.2f "), DistancePriorityScale);
		}
		if (ReplicationPeriodFrame != DefaultValues.ReplicationPeriodFrame)
		{
			Str += FString::Printf(TEXT("ReplicationPeriodFrame: %d "), ReplicationPeriodFrame);
		}
		if (FastPath_ReplicationPeriodFrame != DefaultValues.FastPath_ReplicationPeriodFrame)
		{
			Str += FString::Printf(TEXT("FastPath_ReplicationPeriodFrame: %d "), FastPath_ReplicationPeriodFrame);
		}
		if (ActorChannelFrameTimeout != DefaultValues.ActorChannelFrameTimeout)
		{
			Str += FString::Printf(TEXT("ActorChannelFrameTimeout: %d "), ActorChannelFrameTimeout);
		}
		if (FastSharedReplicationFunc)
		{
			Str += FString::Printf(TEXT("FastSharedReplicationFunc is SET."));
		}

		return Str;
	}

private:

	UPROPERTY()
	float CullDistance = 0.0f;
	UPROPERTY()
	float CullDistanceSquared = 0.f;

};

struct FGlobalActorReplicationInfo;

struct FFastSharedReplicationInfo
{
	uint32 LastAttemptBuildFrameNum = 0; // the last frame we called FastSharedReplicationFunc on
	uint32 LastBunchBuildFrameNum = 0;	// the last frame a new bunch was actually created
	FOutBunch Bunch;

	void CountBytes(FArchive& Ar) const
	{
		Bunch.CountMemory(Ar);
	}
};

DECLARE_MULTICAST_DELEGATE_FourParams(FNotifyActorChangeDormancy, FActorRepListType, FGlobalActorReplicationInfo&, ENetDormancy /*NewVlue*/, ENetDormancy /*OldValue*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FNotifyActorFlushDormancy, FActorRepListType, FGlobalActorReplicationInfo&);

#if REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE
DECLARE_MULTICAST_DELEGATE_TwoParams(FNotifyActorForceNetUpdate, FActorRepListType, FGlobalActorReplicationInfo&);
#endif // REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE

struct FGlobalActorReplicationEvents
{
	FNotifyActorChangeDormancy	DormancyChange;
	FNotifyActorFlushDormancy	DormancyFlush; // This delegate is cleared after broadcasting
#if REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE
	FNotifyActorForceNetUpdate	ForceNetUpdate;
#endif // REPGRAPH_ENABLE_FORCENETUPDATE_DELEGATE
};

/** Per-Actor data that is global for the entire Replication Graph */
struct FGlobalActorReplicationInfo
{
	FGlobalActorReplicationInfo(FClassReplicationInfo& ClassInfo) 
		: LastPreReplicationFrame(0)
		, WorldLocation(ForceInitToZero)
		, bWantsToBeDormant(false)
		, bSwapRolesOnReplicate(false)
		, bWasWorldLocClamped(false)
		, Settings(ClassInfo) 
	{ }

	// -----------------------------------------------------------
	//	Dynamic state
	// -----------------------------------------------------------

	/** The last time AActor::PreReplication was called. Used to track when we need to call it again. */
	uint32 LastPreReplicationFrame = 0;

	/** the last time game code called ForceNetUpdate on this actor */
	uint32 ForceNetUpdateFrame = 0;

	/** Cached World Location of the actor */
	FVector WorldLocation;

	/** Mirrors AActor::NetDormancy > DORM_Awake */
	uint32 bWantsToBeDormant : 1;

	/** True if we should swap the actor role and remote role before calling ReplicateActor() */
	uint32 bSwapRolesOnReplicate : 1;

	/** Set when the object is found with an invalid world location */
	uint32 bWasWorldLocClamped : 1;

	/** Class default mirrors: state that is initialized directly from class defaults (and can be later changed on a per-actor basis) */
	FClassReplicationInfo Settings;
	
	/**	Fast Shared path data */
	TUniquePtr<FFastSharedReplicationInfo> FastSharedReplicationInfo;

	/** Last frame FlushNetDormancy was called. Used to early out when it is called multiple times per frame */
	uint32 LastFlushNetDormancyFrame = 0;

	// -----------------------------------------------------------
	//	Events: Keep this last/at the bottom of the structure. The event data is the largest chunk but accessed the least
	// -----------------------------------------------------------
	FGlobalActorReplicationEvents Events;

	void LogDebugString(FOutputDevice& Ar) const;

	void CountBytes(FArchive& Ar) const
	{
		// Note, we don't count DependentActorList because it's memory will be cached by the allocator / pooling stuff.
		if (FastSharedReplicationInfo.IsValid())
		{
			Ar.CountBytes(sizeof(FFastSharedReplicationInfo), sizeof(FFastSharedReplicationInfo));
			FastSharedReplicationInfo->CountBytes(Ar);
		}

		DependentActorList.CountBytes(Ar);
		ParentActorList.CountBytes(Ar);
	}

	/** Gather the list of dependent actors filtered by the connection's level visibility */
	inline void GatherDependentActorLists(const UNetReplicationGraphConnection& ConnectionManager, FGatheredReplicationActorLists& OutGatheredList)
	{
		DependentActorList.Gather(ConnectionManager, OutGatheredList);
	}

	/** Update actor to new level */
	void NotifyActorRenamed(AActor* Actor, FName PreviousLevelName);

	typedef TArray<FActorRepListType> FDependantListType;

	UE_DEPRECATED(5.2, "The dependent actors are exposed via GatherDependentActorLists now.")
	const FGlobalActorReplicationInfo::FDependantListType& GetDependentActorList()
	{
		
		static FDependantListType EmptyList;
		return EmptyList;
	}

	friend struct FGlobalActorReplicationInfoMap;

private:
	/** When this actor replicates, we replicate these actors immediately afterwards if the client has their level loaded too and the net updated frequency is not too quick */
	FLevelBasedActorList DependentActorList;

	/** When this actor is added to the dependent list of a parent, track the parent here */
	TArray<AActor*> ParentActorList;
};

/** Templatd struct for mapping UClasses to some data type. The main things this provides is that if a UClass* was not explicitly added, it will climb the class heirachy and find the best match (and then store this for faster lookup next time) */
template<typename ValueType>
struct TClassMap
{
	/** Returns ClassInfo for a given class. */
	ValueType& GetChecked(UClass* Class)
	{
		ValueType* Ptr = Get(Class);
		repCheckf(Ptr, TEXT("No ClassInfo found for %s"), *GetNameSafe(Class));
		return *Ptr;
	}

	ValueType* Get(UClass* Class)
	{
		FObjectKey ObjKey(Class);
		if (ValueType* Ptr = Map.Find(ObjKey))
		{
			return Ptr;
		}

		// Allow user to init new data
		if (InitNewElement)
		{
			ValueType NewValue;

			if (InitNewElement(Class, NewValue))
			{
				ValueType& NewData = Map.Emplace(ObjKey, MoveTemp(NewValue));
				return &NewData;
			}
		}


		// We haven't seen this class before, look it up (slower)
		return GetClassInfoForNewClass_r(ObjKey, Class);
	}

	/** Just finds element. Does not climb class hierarchy is explicit entry is not found. */
	const ValueType* FindWithoutClassRecursion(UClass* Class) const
	{
		return Map.Find(Class);
	}

	/** Returns if class has data in the map.  */
	bool Contains(const UClass* Class, bool bIncludeSuperClasses) const
	{
		if (bIncludeSuperClasses)
		{
			while(Class)
			{
				if (Map.Find(FObjectKey(Class)) != nullptr)
				{
					return true;
				}

				Class = Class->GetSuperClass();
			}
			return false;
		}
		
		return (Map.Contains(FObjectKey(Class)));
	}

	/** Sets class info for a given class. Call this in your Replication Graphs setup */
	void Set(UClass* InClass, const ValueType& Value)
	{
		Map.Emplace(FObjectKey(InClass), Value);
		
		// Sets value for all derived classes. This is probably not useful since all classes may not be loaded anyways. TClassMap will 
		// climb the class hierarchy when it encounters a new request. This shouldn't be too expensive, so the lazy approach seems better.
		/*
		TArray<UClass*> Classes = { InClass };
		GetDerivedClasses(InClass, Classes, true);
		for (UClass* Class : Classes)
		{
			UE_LOGF(LogTemp, Display, "Adding for %ls [From %ls]", *GetNameSafe(Class), *GetNameSafe(InClass));
			Map.Add(FObjectKey(Class)) = Value;
		}*/
	}

	void Emplace(UClass* InClass, ValueType&& Value)
	{
		Map.Emplace(FObjectKey(InClass), MoveTemp(Value));
	}

	inline typename TMap<FObjectKey, ValueType>::TIterator CreateIterator() { return Map.CreateIterator(); }
	inline void Reset() { Map.Reset(); }

	TFunction<bool(UClass*, ValueType&)>	InitNewElement;

	void CountBytes(FArchive& Ar) const
	{
		Map.CountBytes(Ar);
	}

private:

	ValueType* GetClassInfoForNewClass_r(FObjectKey OriginalObjKey, const UClass* OriginalClass)
	{
		const UClass* Class = OriginalClass->GetSuperClass();
		if (!Class)
		{
			return nullptr;
		}

		FObjectKey ObjKey(Class);

		if (ValueType* Ptr = Map.Find(ObjKey))
		{
			// Set the original class's data to the super that we found (so that we don't have to climb the chain next time)
			// Need to make a local copy in case the add reallocates the internal TMap data. This prevents Gil writing a blog post about you.
			ValueType LocalData = *Ptr;
			ValueType& NewData = Map.Emplace(OriginalObjKey, LocalData);
			return &NewData;
		}

		return GetClassInfoForNewClass_r(ObjKey, Class);
	}

	TMap<FObjectKey /** UClass */, ValueType> Map;
};

struct FGlobalActorReplicationInfoMap
{
	FGlobalActorReplicationInfoMap()
	{
		// Initialize so we always at least have AActor defined with default values. 
		// It is ok to override this by calling SetClassInfo again.
		ClassMap.Set(AActor::StaticClass(), FClassReplicationInfo());
	}

	/** Returns data associated with the actor. Will create it with default class values if necessary  */
	FGlobalActorReplicationInfo& Get(const FActorRepListType& Actor)
	{
		// Quick lookup - this is the most common case
		if (TUniquePtr<FGlobalActorReplicationInfo>* Ptr = ActorMap.Find(Actor))
		{
			return *Ptr->Get();
		}

		ensureMsgf(IsActorValidForReplication_LogMoreInfo(Actor), TEXT("An invalid actor pointer is passed to FGlobalActorReplicationInfo::Get(), storing this data will generate stale data in the map."));

		// We need to add data for this actor
		FClassReplicationInfo& ClassInfo = GetClassInfo( GetActorRepListTypeClass(Actor) );


		FGlobalActorReplicationInfo* NewGlobalActorRepInfo = new FGlobalActorReplicationInfo(ClassInfo);
		ActorMap.Emplace(Actor, TUniquePtr<FGlobalActorReplicationInfo>(NewGlobalActorRepInfo));
		return *NewGlobalActorRepInfo;
	}

	// Same as above but outputs bool if it was created. This is uncommonly called. Don't want to slow down the frequently called version
	FGlobalActorReplicationInfo& Get(const FActorRepListType& Actor, bool& bWasCreated)
	{
		// Quick lookup - this is the most common case
		if (TUniquePtr<FGlobalActorReplicationInfo>* Ptr = ActorMap.Find(Actor))
		{
			return *Ptr->Get();
		}

		ensureMsgf(IsActorValidForReplication_LogMoreInfo(Actor), TEXT("An invalid actor pointer is passed to FGlobalActorReplicationInfo::Get(), storing this data will generate stale data in the map."));

		bWasCreated = true;

		// We need to add data for this actor
		FClassReplicationInfo& ClassInfo = GetClassInfo( GetActorRepListTypeClass(Actor) );

		FGlobalActorReplicationInfo* NewGlobalActorRepInfo = new FGlobalActorReplicationInfo(ClassInfo);
		ActorMap.Emplace(Actor, TUniquePtr<FGlobalActorReplicationInfo>(NewGlobalActorRepInfo));
		return *NewGlobalActorRepInfo;
	}

	void SetInitClassInfoFunc(TFunction<bool(UClass*, FClassReplicationInfo&)> Func)
	{
		ClassMap.InitNewElement = Func;
	}


	/** Finds data associated with the actor but does not create if its not there yet. */
	inline FGlobalActorReplicationInfo* Find(const FActorRepListType& Actor) const 
	{
		if (const TUniquePtr<FGlobalActorReplicationInfo>* Ptr = ActorMap.Find(Actor))
		{
			return Ptr->Get();
		}

		return nullptr;
	}

	/** Removes actor data from map */
	REPLICATIONGRAPH_API int32 Remove(const FActorRepListType& RemovedActor);

	/** Update actor to new level */
	void NotifyActorRenamed(AActor* Actor, FName PreviousStreamingLevelName);
	
	/** Returns ClassInfo for a given class. */
	inline FClassReplicationInfo& GetClassInfo(UClass* Class) { return ClassMap.GetChecked(Class); }

	/** Sets class info for a given class and its derived classes if desired. Call this in your Replication Graphs setup */
	inline void SetClassInfo(UClass* InClass, const FClassReplicationInfo& Info) {	ClassMap.Set(InClass, Info); }
	
	inline TMap<FActorRepListType, TUniquePtr<FGlobalActorReplicationInfo>>::TIterator CreateActorMapIterator() { return ActorMap.CreateIterator(); }
	inline TMap<FActorRepListType, TUniquePtr<FGlobalActorReplicationInfo>>::TConstIterator CreateActorMapIterator() const { return ActorMap.CreateConstIterator(); }
	inline TMap<FObjectKey, FClassReplicationInfo>::TIterator CreateClassMapIterator() { return ClassMap.CreateIterator(); }

	int32 Num() const { return ActorMap.Num(); }

	inline void ResetActorMap() { ActorMap.Reset(); }

	void CountBytes(FArchive& Ar) const
	{
		ActorMap.CountBytes(Ar);
		for (const auto& KVP : ActorMap)
		{
			if (KVP.Value.IsValid())
			{
				Ar.CountBytes(sizeof(FGlobalActorReplicationInfo), sizeof(FGlobalActorReplicationInfo));
				KVP.Value->CountBytes(Ar);
			}
		}

		ClassMap.CountBytes(Ar);
	}

	enum class EWarnFlag : uint8
	{
		None = 0,
		WarnAlreadyDependant = 1 << 0,
		WarnParentNotRegistered = 1 << 1,
		AllWarnings = WarnAlreadyDependant | WarnParentNotRegistered
	};

	/**
	* AddDependentActor makes a dependent actor (the child) considered for replication whenever the parent actor is replicated to a client.
	* When the parent is replicated, the child actor will be also replicated even if it is outside it's net cull distance.
	* The child will NOT be replicated if it has no data to send or the net update frequency is too quick since the last netupdate.
	* Note that this link is one way only, meaning if a child replicates by itself it will not force it's parents to be replicated at the same time.
	* When setting a dependent link, it is the caller's responsibility to remove the dependency if one of the actor gets destroyed.
	* 
	* @param Parent Actor that will try to also replicate the child.
	* @param Child The actor that we will try to replicate when the parent actor is replicated.
	* @param WarnFlag Set a flag to trigger an ensure and detect when two actors get set dependent multiple times.
	*/
	REPLICATIONGRAPH_API void AddDependentActor(AActor* Parent, AActor* Child, FGlobalActorReplicationInfoMap::EWarnFlag WarnFlag = FGlobalActorReplicationInfoMap::EWarnFlag::None);

	/** 
	* RemoveDependentActor will remove the given child actor from the parent's dependent list.
	* This will stop making the child actor try to be replicated whenever the parent actor is
	*/
	void RemoveDependentActor(AActor* Parent, AActor* Child)
	{
		if (Parent && Child)
		{
			if (FGlobalActorReplicationInfo* ParentInfo = Find(Parent))
			{
				ParentInfo->DependentActorList.RemoveNetworkActor(Child);
			}

			if (FGlobalActorReplicationInfo* ChildInfo = Find(Child))
			{
				ChildInfo->ParentActorList.RemoveSingleSwap(Parent);
			}
		}
	}

	void RemoveAllActorDependencies(AActor* MainActor, FGlobalActorReplicationInfo* MainActorInfo)
	{
		check(MainActorInfo);

		FGatheredReplicationActorLists ListContainer;
		MainActorInfo->DependentActorList.AppendAllLists(ListContainer);

		// Remove the actor from its child dependents
		const TArrayView<const FActorRepListType> DependentActorsList = ListContainer.ViewActors(EActorRepListTypeFlags::Default);
		for (AActor* DependentActor : DependentActorsList)
		{
			if (FGlobalActorReplicationInfo* ChildInfo = Find(DependentActor))
			{
				ChildInfo->ParentActorList.RemoveSingleSwap(MainActor);
			}
		}

		// Remove the actor from its parents
		FNewReplicatedActorInfo LevelActorInfo(MainActor);

		for (AActor* ParentActor : MainActorInfo->ParentActorList)
		{
			if (FGlobalActorReplicationInfo* ParentInfo = Find(ParentActor))
			{
				ParentInfo->DependentActorList.RemoveNetworkActor(LevelActorInfo);
			}
		}

		MainActorInfo->DependentActorList.Reset();
		MainActorInfo->ParentActorList.Reset();
	}

	UE_DEPRECATED(5.2, "Fixed a typo. Use RemoveAllActorDependencies instead")
	void RemoveAllActorDependancies(AActor* MainActor)
	{
		RemoveAllActorDependencies(MainActor);
	}

	void RemoveAllActorDependencies(AActor* MainActor)
	{
		FGlobalActorReplicationInfo* MainActorInfo = Find(MainActor);
		if (MainActor && MainActorInfo)
		{
			RemoveAllActorDependencies(MainActor, MainActorInfo);
		}
	}

private:

	TMap<FActorRepListType, TUniquePtr<FGlobalActorReplicationInfo>> ActorMap;
	TClassMap<FClassReplicationInfo> ClassMap;
};

ENUM_CLASS_FLAGS(FGlobalActorReplicationInfoMap::EWarnFlag);

/** Per-Actor data that is stored per connection */
struct FConnectionReplicationActorInfo
{
	FConnectionReplicationActorInfo() : bDormantOnConnection(0), bTearOff(0), bGridSpatilization_AlreadyDormant(0), bForceCullDistanceToZero(0) { }

	FConnectionReplicationActorInfo(const FGlobalActorReplicationInfo& GlobalInfo) : bDormantOnConnection(0), bTearOff(0), bGridSpatilization_AlreadyDormant(0), bForceCullDistanceToZero(0)
	{
		// Pull data from the global actor info. This is done for things that we just want to duplicate in both places so that we can avoid a lookup into the global map
		// and also for things that we want to be overridden per (connection/actor)

		ReplicationPeriodFrame = GlobalInfo.Settings.ReplicationPeriodFrame;
		FastPath_ReplicationPeriodFrame = GlobalInfo.Settings.FastPath_ReplicationPeriodFrame;
		SetCullDistanceSquared(GlobalInfo.Settings.GetCullDistanceSquared());
	}

	/** Resets the data, except for the "settings" data that we pulled from GlobalInfo */
	void ResetFrameCounters()
	{
		Channel = nullptr;
		NextReplicationFrameNum = 0;
		LastRepFrameNum = 0;
		ActorChannelCloseFrameNum = 0;		

		FastPath_NextReplicationFrameNum = 0;
		FastPath_LastRepFrameNum = 0;

		// Note: purposefully not clearing bDormantOnConnection or bTearOff.
	}

	void SetCullDistanceSquared(float InCullDistanceSquared)
	{
		CullDistanceSquared = InCullDistanceSquared;
		CullDistance = FMath::Sqrt(CullDistanceSquared);
	}

	float GetCullDistance() const { return bForceCullDistanceToZero ? 0.0f : CullDistance; }
	float GetCullDistanceSquared() const { return bForceCullDistanceToZero ? 0.0f : CullDistanceSquared; }

	UActorChannel* Channel = nullptr;

private:
	float CullDistance = 0.f;
	float CullDistanceSquared = 0.f;

public:
	
	/** Default replication */
	uint32	NextReplicationFrameNum = 0;	/** The next frame we are allowed to replicate on */
	uint32	LastRepFrameNum = 0;			/** The last frame that this actor replicated on to this connection */

	/** FastPath versions of the above */
	uint32	FastPath_NextReplicationFrameNum = 0;
	uint32	FastPath_LastRepFrameNum = 0;

	/** Min frames that have to pass between subsequent calls to ReplicateActor */
	uint16	ReplicationPeriodFrame = 1;		
	uint16	FastPath_ReplicationPeriodFrame = 1;
	
	/** The frame num that we will close the actor channel. This will get updated/pushed anytime the actor replicates based on FGlobalActorReplicationInfo::ActorChannelFrameTimeout  */
	uint32 ActorChannelCloseFrameNum = 0;

	uint8 bDormantOnConnection:1;
	uint8 bTearOff:1;

	/** Used as an optimization when doing 2D Grid Spatilization, prevents replicating the dormancy of the same actor twice in splitscreen evaluations. */
	uint8 bGridSpatilization_AlreadyDormant:1;

	/** When enabled: GetCullDistance() and GetCullDistanceSquared() will return 0.0f.
	*	When disabled: GetCullDistance() and GetCullDistanceSquared() return CullDistance and CullDistanceSquared respectively.
	*	Useful for temporarily making the actor on this connection always relevant. */
	uint8 bForceCullDistanceToZero:1;

	void LogDebugString(FOutputDevice& Ar) const;
};

/** Map for Actor -> ConnectionActorInfo. This wraps the TMap mainly so we can do custom initialization in FindOrAdd. */
struct FPerConnectionActorInfoMap
{
	inline FConnectionReplicationActorInfo& FindOrAdd(const FActorRepListType& Actor)
	{
		if (TSharedPtr<FConnectionReplicationActorInfo>* ValuePtr = ActorMap.Find(Actor))
		{
			return *ValuePtr->Get();
		}

		FConnectionReplicationActorInfo* NewInfo = new FConnectionReplicationActorInfo(GlobalMap->Get(Actor));
		ActorMap.Emplace(Actor, TSharedPtr<FConnectionReplicationActorInfo>(NewInfo) );
		return *NewInfo;
	}

	inline FConnectionReplicationActorInfo* Find(const FActorRepListType& Actor)
	{
		if (TSharedPtr<FConnectionReplicationActorInfo>* ValuePtr = ActorMap.Find(Actor))
		{
			return ValuePtr->Get();
		}

		return nullptr;
	}

	inline FConnectionReplicationActorInfo* FindByChannel(UActorChannel* Channel)
	{
		if (TSharedPtr<FConnectionReplicationActorInfo>* ValuePtr = ChannelMap.Find(Channel))
		{
			return ValuePtr->Get();
		}

		return nullptr;
	}

	inline void AddChannel(const FActorRepListType& Actor, UActorChannel* Channel)
	{
		if (TSharedPtr<FConnectionReplicationActorInfo>* ValuePtr = ActorMap.Find(Actor))
		{
			ChannelMap.Add(Channel, *ValuePtr);
		}
	}

	inline void RemoveChannel(UActorChannel* Channel)
	{
		ChannelMap.Remove(Channel);
	}

	inline void RemoveActor(const FActorRepListType& Actor)
	{
		ActorMap.Remove(Actor);
	}

	inline TMap<FActorRepListType, TSharedPtr<FConnectionReplicationActorInfo>>::TIterator CreateIterator()
	{
		return ActorMap.CreateIterator();
	}

	inline void SetGlobalMap(FGlobalActorReplicationInfoMap* InGlobalMap)
	{
		GlobalMap = InGlobalMap;
	}

	inline TMap<UActorChannel*, TSharedPtr<FConnectionReplicationActorInfo>>::TIterator CreateChannelIterator()
	{
		return ChannelMap.CreateIterator();
	}

	void ResetActorMap()
	{
		ActorMap.Reset();
		ChannelMap.Reset();
	}

	int32 Num() const { return ActorMap.Num(); }

	void CountBytes(FArchive& Ar) const
	{
		TSet<FConnectionReplicationActorInfo*> UniqueInfos;

		ActorMap.CountBytes(Ar);
		ChannelMap.CountBytes(Ar);

		for (const auto& KVP : ActorMap)
		{
			UniqueInfos.Add(KVP.Value.Get());
		}

		for (const auto& KVP : ChannelMap)
		{
			UniqueInfos.Add(KVP.Value.Get());
		}

		SIZE_T Size = sizeof(FConnectionReplicationActorInfo) * UniqueInfos.Num();
		Ar.CountBytes(Size, Size);
	}

private:
	TMap<FActorRepListType, TSharedPtr<FConnectionReplicationActorInfo>> ActorMap;
	TMap<UActorChannel*, TSharedPtr<FConnectionReplicationActorInfo>> ChannelMap;
	FGlobalActorReplicationInfoMap* GlobalMap;
};

/** Data that every replication graph has access to/is initialized with */
struct FReplicationGraphGlobalData
{
	FReplicationGraphGlobalData() { }
	FReplicationGraphGlobalData(FGlobalActorReplicationInfoMap* InRepMap, UWorld* InWorld, UReplicationGraph* InReplicationGraph) : GlobalActorReplicationInfoMap(InRepMap), World(InWorld), ReplicationGraph(InReplicationGraph) { }

	FGlobalActorReplicationInfoMap* GlobalActorReplicationInfoMap = nullptr;

	UWorld* World = nullptr;

	UReplicationGraph* ReplicationGraph = nullptr;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Prioritized Actor Lists
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

/** Stores "full debug details" about how an actor was prioritized. This is not used in the actual replication code, just saved off for logging/debugging.  */
struct FPrioritizedActorFullDebugDetails
{
	explicit FPrioritizedActorFullDebugDetails(FActorRepListType InActor) : Actor(InActor) { }
	friend bool operator==(const FPrioritizedActorFullDebugDetails& Details, const FActorRepListType& InActor)
	{
		return Details.Actor == InActor;
	}

	FActorRepListType Actor;
	FVector::FReal DistanceSq = 0.f;
	float DistanceFactor = 0.f;

	uint32 FramesSinceLastRap = 0;
	float StarvationFactor = 0.f;

	uint32 ForceNetUpdateDelta = 0;
	float GameCodeScaling = 0.f;

	FString BuildString() const
	{
		// This is pretty stupid slow but only needed for debug/logging
		FString Str;
		auto Append = [&Str](FString InStr) { Str += FString::Printf(TEXT("%-30s "), *InStr); };
		
		Append( DistanceFactor > 0.f ? FString::Printf(TEXT("(Dist: %.2f | %.2f) "), FMath::Sqrt(DistanceSq), DistanceFactor) : TEXT(""));
		Append( StarvationFactor > 0.f ? FString::Printf(TEXT("(FramesLastRep: %d | %.2f) "), FramesSinceLastRap, StarvationFactor) : TEXT("") );
		Append ( GameCodeScaling > 0.f ? FString::Printf(TEXT("(ForceNetUpdate: %d | %.2f) "), ForceNetUpdateDelta, GameCodeScaling) : TEXT(""));
		
		return Str;
	}
};

/** Debug data about an actor that was skipped during the prioritization phase */
struct FSkippedActorFullDebugDetails
{
	explicit FSkippedActorFullDebugDetails(FActorRepListType InActor) : Actor(InActor) { }
	FActorRepListType Actor;
	bool bWasDormant = false; // If set, was skipped because it is dormant on this connection
	float DistanceCulled = 0.f; // If set, was skipped due to distance culling
	uint32 FramesTillNextReplication = 0; // If set, was skipped due to not being ready for replication
};

/** Prioritized List of actors to replicate. This is what we actually use to replicate actors. */
struct FPrioritizedRepList
{
	FPrioritizedRepList() = default;
	explicit FPrioritizedRepList(const FPrioritizedRepList& Other) { Items = Other.Items; }

	struct FItem
	{
		FItem(float InPriority, FActorRepListType InActor, FGlobalActorReplicationInfo* InGlobal, FConnectionReplicationActorInfo* InConn) 
			: Priority(InPriority), Actor(InActor) , GlobalData(InGlobal), ConnectionData(InConn) { }
		bool operator<(const FItem& Other) const { return Priority < Other.Priority; }

		float Priority;
		FActorRepListType Actor;
		
		FGlobalActorReplicationInfo* GlobalData;
		FConnectionReplicationActorInfo* ConnectionData;
	};

	TArray<FItem> Items;

	void CountBytes(FArchive& Ar) const
	{
		Items.CountBytes(Ar);

#if REPGRAPH_DETAILS
		if (FullDebugDetails.IsValid())
		{
			Ar.CountBytes(sizeof(TArray<FPrioritizedActorFullDebugDetails>), sizeof(TArray<FPrioritizedActorFullDebugDetails>));
			FullDebugDetails->CountBytes(Ar);
		}

		if (SkippedDebugDetails.IsValid())
		{
			Ar.CountBytes(sizeof(TArray<FSkippedActorFullDebugDetails>), sizeof(TArray<FSkippedActorFullDebugDetails>));
			SkippedDebugDetails->CountBytes(Ar);
		}
#endif
	}

	void Reset()
	{
		Items.Reset();
#if REPGRAPH_DETAILS
		FullDebugDetails.Reset();
		SkippedDebugDetails.Reset();
#endif
	}
	
#if REPGRAPH_DETAILS
	FPrioritizedActorFullDebugDetails* GetNextFullDebugDetails(FActorRepListType Actor)
	{
		if (FullDebugDetails.IsValid() == false)
		{
			FullDebugDetails = MakeUnique<TArray<FPrioritizedActorFullDebugDetails> >();
		}
		return new (*FullDebugDetails) FPrioritizedActorFullDebugDetails(Actor);
	}
	TUniquePtr<TArray<FPrioritizedActorFullDebugDetails> > FullDebugDetails;
	
	
	FSkippedActorFullDebugDetails* GetNextSkippedDebugDetails(FActorRepListType Actor)
	{
		if (SkippedDebugDetails.IsValid() == false)
		{
			SkippedDebugDetails = MakeUnique<TArray<FSkippedActorFullDebugDetails> >();
		}
		return new (*SkippedDebugDetails) FSkippedActorFullDebugDetails(Actor);
	}
	TUniquePtr<TArray<FSkippedActorFullDebugDetails> > SkippedDebugDetails;
#endif
};
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Connection Gather Actor List Parameters
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
#if !defined(REPGRAPH_VIEWERS_PER_CONNECTION)
#define REPGRAPH_VIEWERS_PER_CONNECTION 4
#endif

// Allocator for a connection and all of its sub connections
typedef TInlineAllocator<REPGRAPH_VIEWERS_PER_CONNECTION> FReplicationGraphConnectionsAllocator;
typedef TArray<FNetViewer, FReplicationGraphConnectionsAllocator> FNetViewerArray;

// Parameter structure for what we actually pass down during the Gather phase.
struct FConnectionGatherActorListParameters
{
	FConnectionGatherActorListParameters(
			FNetViewerArray& InViewers,
			UNetReplicationGraphConnection& InConnectionManager,
			const TSet<FName>& InClientVisibleLevelNamesRef,
			uint32 InReplicationFrameNum,
			FGatheredReplicationActorLists& InOutGatheredReplicationLists,
			bool bInSelectedForHeavyComputation)
		: Viewers(InViewers)
		, ConnectionManager(InConnectionManager)
		, ReplicationFrameNum(InReplicationFrameNum)
		, OutGatheredReplicationLists(InOutGatheredReplicationLists)
		, ClientVisibleLevelNamesRef(InClientVisibleLevelNamesRef)
		, bIsSelectedForHeavyComputation(bInSelectedForHeavyComputation)
	{
	}

	FNetViewerArray Viewers;
	UNetReplicationGraphConnection& ConnectionManager;
	uint32 ReplicationFrameNum;

	/** Out: The data nodes are going to add to */
	FGatheredReplicationActorLists& OutGatheredReplicationLists;

	bool CheckClientVisibilityForLevel(const FName& StreamingLevelName) const
	{
		if (StreamingLevelName == LastCheckedVisibleLevelName)
		{
			return true;
		}

		const bool bVisible = ClientVisibleLevelNamesRef.Contains(StreamingLevelName);
		if (bVisible)
		{
			LastCheckedVisibleLevelName = StreamingLevelName;
		}
		return bVisible;
	}



	// Cached off reference for fast Level Visibility lookup
	const TSet<FName>& ClientVisibleLevelNamesRef;
	const bool bIsSelectedForHeavyComputation;

private:

	mutable FName LastCheckedVisibleLevelName;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// RPCs
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

struct FRPCSendPolicyInfo
{
	FRPCSendPolicyInfo(const bool bInSendImmediately) : bSendImmediately(bInSendImmediately) { }

	/** 
	 * When true this will modify the send behavior of an RPC.
	 * If the RPC is called from inside NetDriver::TickDispatch (aka during packet reception and remote RPC execution) then the RPC will get immediately 
	 * sent to remote clients at the end of TickDispatch before the regular game tick.
	 * This reduces latency since normally RPCs are sent alongside normal replicated data at the end of the frame during NetDriver::TickFlush.
	 * The drawbacks are extra bandwidth cost and higher CPU usage.
	 * 
	 * Note that with Unreliable Multicasts, this flag changes their order when executed on the remote client.
	 * By default only Unreliable Multicasts are executed after replicated properties have been applied on the simulated actor.
	 * With this flag they get executed before any replicated properties are applied.
	 */
	uint8 bSendImmediately:1;

	// Suspect that this will grow over time. Possibly things like "min distance to send immediately" etc
};

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Debug Info
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

struct FReplicationGraphDebugInfo
{
	FReplicationGraphDebugInfo( FOutputDevice& InAr ) : Ar(InAr) { }

	enum EFlags
	{
		ShowActors,
		ShowClasses,
		ShowNativeClasses,
		ShowTotalCount
	};
	
	FOutputDevice& Ar;
	EFlags Flags;
	bool bShowEmptyNodes = false;
	FString CurrentIndentString;
	const FString IndentString = TEXT("  ");

	void Log(const FString& Str) { Ar.Logf(TEXT("%s%s"), *CurrentIndentString, *Str); }

	void PushIndent() { CurrentIndentString += IndentString; }
	void PopIndent() { CurrentIndentString.LeftChopInline(IndentString.Len(), EAllowShrinking::No); }
};

REPLICATIONGRAPH_API void LogActorRepList(FReplicationGraphDebugInfo& DebugInfo, FString Prefix, const FActorRepListRefView& List);

struct FPrioritizedActorDebugInfo
{
	int32 DormantCount = 0;
	int32 NotReadyCount = 0;
	int32 DistanceCulledCount = 0;
	int32 ReplicatedCount = 0;
	int32 StarvedCount = 0;
};


struct FNativeClassAccumulator
{
	void Increment(UClass* Class)
	{
		while (Class)
		{
			if (Class->IsNative())
			{
				break;
			}
			Class = Class->GetSuperClass();
		}
		
		Map.FindOrAdd(Class)++;
	}

	FString BuildString()
	{
		FString Str;
		Sort();
		for (auto& It : Map)
		{
			Str += FString::Printf(TEXT("[%s, %d] "), *It.Key->GetName(), It.Value);
		}
		return Str;
	}

	void CountBytes(FArchive& Ar) const
	{
		Map.CountBytes(Ar);
	}

	void Reset() { Map.Reset(); }
	void Sort() { Map.ValueSort(TGreater<int32>()); }
	TMap<UClass*, int32> Map;
};

#if WITH_EDITOR
void ForEachClientPIEWorld(TFunction<void(UWorld*)> Func);
#else
inline void ForEachClientPIEWorld(TFunction<void(UWorld*)> Func) { }
#endif

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// Profiling
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------


CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphMS);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphKBytes);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphChannelsOpened);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphNumReps);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphVisibleLevels);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphForcedUpdates);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphCleanMS);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphCleanNumReps);
CSV_DECLARE_CATEGORY_EXTERN(ReplicationGraphRedundantMS);

#ifndef REPGRAPH_CSV_TRACKER
#define REPGRAPH_CSV_TRACKER (CSV_PROFILER_STATS && WITH_SERVER_CODE)
#endif

/** Helper struct for tracking finer grained ReplicationGraph stats through the CSV profiler. Intention is that it is setup/configured in the UReplicationGraph subclasses */
struct FReplicationGraphCSVTracker
{
	enum class EActorFlags : uint8
	{
		None = 0,
		IsInDiscovery = 1 << 0,
		AlreadyReplicatedThisFrame = 1 << 1
	};

	FRIEND_ENUM_CLASS_FLAGS(EActorFlags)

	REPLICATIONGRAPH_API FReplicationGraphCSVTracker();

	void TearDown()
	{
#if REPGRAPH_CSV_TRACKER
		if (UntrackedReplications.Num())
		{
			UE_LOGF(LogReplicationGraph, Log, "Untracked CSV Classes:");

			UntrackedReplications.ValueSort(TGreater<uint32>());

			for (const TPair<TObjectKey<UClass>, uint32>& Untracked : UntrackedReplications)
			{
				UE_LOGF(LogReplicationGraph, Log, "    %ls : %u", *GetNameSafe(Untracked.Key.ResolveObjectPtr()), Untracked.Value);
			}
		}
#endif
	}

	/** Tracks an explicitly set class. This does NOT include child classes! This is the fastest stat and should be fine to enable in shipping/test builds.  */
	void SetExplicitClassTracking(UClass* ExactActorClass, const FString& StatNamePrefix)
	{
		if (!ExplicitClassTracker.Contains(ExactActorClass))
		{
			ExplicitClassTracker.Emplace(ExactActorClass, StatNamePrefix);
		}
	}

	/** Sets explicit class tracking for fast/shared path replication. Does not include base classes */
	void SetExplicitClassTracking_FastPath(UClass* ExactActorClass, const FString& StatNamePrefix)
	{
		if (!ExplicitClassTracker_FastPath.Contains(ExactActorClass))
		{
			FString FinalStrPrefix = TEXT("FastPath_") + StatNamePrefix;
			ExplicitClassTracker_FastPath.Emplace(ExactActorClass, FinalStrPrefix);
		}
	}

	/** Tracks a class and all of its children (under a single stat set). This will be a little slower (TMap lookup) but still probably ok if used in moderation (only track your top 3 or so classes) */
	void SetImplicitClassTracking(UClass* BaseActorClass, const FString& StatNamePrefix)
	{
		TSharedPtr<FTrackedData> NewData = MakeShared<FTrackedData>(StatNamePrefix);
		UniqueImplicitTrackedData.Add(NewData);
		ImplicitClassTracker.Emplace(BaseActorClass, MoveTemp(NewData));
	}

	/** Returns true when the level data is created for the first time */
	bool VisibleLevelConnectionAdded(FName LevelName)
	{
#if REPGRAPH_CSV_TRACKER
		FVisibleLevelData* LevelData = VisibleLevelConnectionTracker.FindByKey(LevelName);
		if (LevelData)
		{
			LevelData->NbConnections++;
			return false;
		}
		else
		{
			FVisibleLevelData VisibleLevelData;
			VisibleLevelData.LevelName = LevelName;
			VisibleLevelData.NbConnections = 1;
			VisibleLevelConnectionTracker.Add(VisibleLevelData);
			return true;
		}
#else
		return false;
#endif
	}

	void VisibleLevelConnectionRemoved(FName LevelName)
	{
#if REPGRAPH_CSV_TRACKER
		if (FVisibleLevelData* LevelData = VisibleLevelConnectionTracker.FindByKey(LevelName))
		{
			LevelData->NbConnections--;
			if (LevelData->NbConnections <= 0)
			{
				VisibleLevelConnectionTracker.RemoveSingleSwap(*LevelData);
			}
		}
#endif
	}

	void SetVisibleLevelCustomName(FName LevelName, const FString& CustomReadableName)
	{
#if REPGRAPH_CSV_TRACKER
		const FName NameReadable(*CustomReadableName);
		FVisibleLevelData* LevelData = VisibleLevelConnectionTracker.FindByKey(LevelName);
		if (LevelData)
		{
			// Note this will create a new stat entry in the csv file
			LevelData->CustomReadableName = NameReadable;
		}
		else
		{
			FVisibleLevelData VisibleLevelData;
			VisibleLevelData.LevelName = LevelName;
			VisibleLevelData.NbConnections = 0;
			VisibleLevelData.CustomReadableName = NameReadable;

			VisibleLevelConnectionTracker.Add(VisibleLevelData);
		}
#endif //REPGRAPH_CSV_TRACKER
	}

	UE_DEPRECATED(5.1, "Use the overload of PostReplicateActor that takes EActorFlags.")
	void PostReplicateActor(UClass* ActorClass, const double Time, const int64 Bits, const bool bIsActorDiscovery)
	{
		const EActorFlags Flags = bIsActorDiscovery ? EActorFlags::IsInDiscovery : EActorFlags::None;

		PostReplicateActor(ActorClass, Time, Bits, Flags);
	}

	void PostReplicateActor(UClass* ActorClass, const double Time, const int64 Bits, const EActorFlags Flags)
	{
#if REPGRAPH_CSV_TRACKER
		if (!bIsCapturing)
		{
			return;
		}

		FTrackedData* TrackedData = ExplicitClassTracker.Find(ActorClass);
		if (TrackedData == nullptr)
		{
			TrackedData = ImplicitClassTracker.GetChecked(ActorClass).Get();
			if (TrackedData == nullptr)
			{
				TrackedData = &EverythingElse;

				if (bReportUntrackedClasses)
				{
					uint32& Count = UntrackedReplications.FindOrAdd(ActorClass);
					++Count;
				}
			}
		}

		// When opening actor channels keep all traffic in a separate bucket
		if (EnumHasAnyFlags(Flags, EActorFlags::IsInDiscovery))
		{
			ActorDiscovery.BitsAccumulated += Bits;
			ActorDiscovery.CPUTimeAccumulated += Time;

			if (EnumHasAnyFlags(Flags, EActorFlags::AlreadyReplicatedThisFrame))
			{
				ActorDiscovery.RedundantRepCPUTimeAccumulated += Time;
			}

			// But keep the number of replicated classes unique
			TrackedData->NumReplications++;
		}
		else
		{
			TrackedData->BitsAccumulated += Bits;
			TrackedData->CPUTimeAccumulated += Time;
			TrackedData->NumReplications++;

			if (Bits == 0)
			{
				TrackedData->CleanCPUTimeAccumulated += Time;
				TrackedData->CleanNumReplications++;
			}

			if (EnumHasAnyFlags(Flags, EActorFlags::AlreadyReplicatedThisFrame))
			{
				TrackedData->RedundantRepCPUTimeAccumulated += Time;
			}
		}
#endif	
	}

	void PostFastPathReplication(UClass* ActorClass, const double Time, const int64 Bits)
	{
#if REPGRAPH_CSV_TRACKER
		if (!bIsCapturing)
		{
			return;
		}

		if (FTrackedData* Data = ExplicitClassTracker_FastPath.Find(ActorClass))
		{
			Data->BitsAccumulated += Bits;
			Data->CPUTimeAccumulated += Time;
			Data->NumReplications++;
		}
		else
		{
			EverythingElse_FastPath.BitsAccumulated += Bits;
			EverythingElse_FastPath.CPUTimeAccumulated += Time;
			EverythingElse_FastPath.NumReplications++;
		}
#endif
	}

	void PostActorChannelCreated(UClass* ActorClass)
	{
#if REPGRAPH_CSV_TRACKER
		if (!bIsCapturing)
		{
			return;
		}

		FTrackedData* TrackedData = ExplicitClassTracker.Find(ActorClass);
		if (TrackedData == nullptr)
		{
			TrackedData = ImplicitClassTracker.GetChecked(ActorClass).Get();
			if (TrackedData == nullptr)
			{
				TrackedData = &EverythingElse;
			}
		}

		TrackedData->ChannelsOpened++;
#endif
	}

	void PostActorForceUpdated(UClass* ActorClass)
	{
#if REPGRAPH_CSV_TRACKER
		if (!bIsCapturing)
		{
			return;
		}

		FTrackedData* TrackedData = ExplicitClassTracker.Find(ActorClass);
		if (TrackedData == nullptr)
		{
			TrackedData = ImplicitClassTracker.GetChecked(ActorClass).Get();
			if (TrackedData == nullptr)
			{
				TrackedData = &EverythingElse;
			}
		}

		TrackedData->ForcedUpdates++;
#endif
	}

	void ResetTrackedClasses()
	{
		ExplicitClassTracker.Reset();
		ImplicitClassTracker.Reset();
		ImplicitClassTracker.Set(AActor::StaticClass(), TSharedPtr<FTrackedData>()); // forces caching of "no tracking" for all other classes
		EverythingElse.Reset();
		EverythingElse_FastPath.Reset();
		ActorDiscovery.Reset();
		VisibleLevelConnectionTracker.Reset();
		UntrackedReplications.Reset();
	}

	void EndReplicationFrame()
	{
#if REPGRAPH_CSV_TRACKER
		FCsvProfiler* Profiler = FCsvProfiler::Get();
		bIsCapturing = Profiler->IsCapturing();
		if (bIsCapturing)
		{
			for (TPair<TObjectKey<UClass>, FTrackedData>& Item : ExplicitClassTracker)
			{
				PushStats(Profiler, Item.Value);	
			}

			for (TPair<TObjectKey<UClass>, FTrackedData>& Item : ExplicitClassTracker_FastPath)
			{
				PushStats(Profiler, Item.Value);
			}

			for (TSharedPtr<FTrackedData>& SharedPtr : UniqueImplicitTrackedData)
			{
				if (FTrackedData* Data = SharedPtr.Get())
				{
					PushStats(Profiler, *Data);
				}
			}

			PushStats(Profiler, EverythingElse);
			PushStats(Profiler, EverythingElse_FastPath);
			PushStats(Profiler, ActorDiscovery);

			for (const FVisibleLevelData& VisibleLevel : VisibleLevelConnectionTracker)
			{
				if (VisibleLevel.CustomReadableName.IsValid())
				{
					Profiler->RecordCustomStat(VisibleLevel.CustomReadableName, CSV_CATEGORY_INDEX(ReplicationGraphVisibleLevels), VisibleLevel.NbConnections, ECsvCustomStatOp::Set);
				}
				else
				{
					Profiler->RecordCustomStat(VisibleLevel.LevelName, CSV_CATEGORY_INDEX(ReplicationGraphVisibleLevels), VisibleLevel.NbConnections, ECsvCustomStatOp::Set);
				}
			}
		}
#endif
	}

	void CountBytes(FArchive& Ar) const
	{
		ExplicitClassTracker.CountBytes(Ar);
		ExplicitClassTracker_FastPath.CountBytes(Ar);
		ImplicitClassTracker.CountBytes(Ar);
		UniqueImplicitTrackedData.CountBytes(Ar);
		VisibleLevelConnectionTracker.CountBytes(Ar);
		UntrackedReplications.CountBytes(Ar);
	}

	void SetReportUntrackedClasses(bool bReport)
	{
		bReportUntrackedClasses = bReport;
	}

public:

	struct FVisibleLevelData
	{
		FName LevelName;
		int32 NbConnections;
		FName CustomReadableName;

		bool operator==(const FVisibleLevelData& Other) const { return LevelName == Other.LevelName; }
		bool operator==(const FName& InLevelName) const { return LevelName == InLevelName; }
	};

	/** */
	const TArray<FReplicationGraphCSVTracker::FVisibleLevelData>& GetVisibleLevelsData() const { return VisibleLevelConnectionTracker; }

private:

	struct FTrackedData
	{
		FTrackedData() = default;

		explicit FTrackedData(FString Suffix)
		{
#if REPGRAPH_CSV_TRACKER
			StatName = FName(*Suffix);
#endif
		}

		double CPUTimeAccumulated = 0.0;
		double CleanCPUTimeAccumulated = 0.0;
		double RedundantRepCPUTimeAccumulated = 0.0;
		int64 BitsAccumulated = 0;
		int32 ChannelsOpened = 0;
		int32 NumReplications = 0;
		int32 CleanNumReplications = 0;
		int32 ForcedUpdates = 0;

		FName StatName;

		void Reset()
		{
			CPUTimeAccumulated = 0.0;
			CleanCPUTimeAccumulated = 0.0;
			RedundantRepCPUTimeAccumulated = 0.0;
			BitsAccumulated = 0;
			ChannelsOpened = 0;
			NumReplications = 0;
			CleanNumReplications = 0;
			ForcedUpdates = 0;
		}
	};

	TMap<TObjectKey<UClass>, FTrackedData> ExplicitClassTracker;
	TMap<TObjectKey<UClass>, FTrackedData> ExplicitClassTracker_FastPath;

	TClassMap<TSharedPtr<FTrackedData>> ImplicitClassTracker;
	
	TArray<TSharedPtr<FTrackedData>> UniqueImplicitTrackedData;

	FTrackedData EverythingElse;
	FTrackedData EverythingElse_FastPath;
	FTrackedData ActorDiscovery;

	TMap<TObjectKey<UClass>, uint32> UntrackedReplications;

	// Counts the number of connections who are currently seeing each level name
	TArray<FVisibleLevelData> VisibleLevelConnectionTracker;

	bool bIsCapturing = false;
	bool bReportUntrackedClasses = false;

#if REPGRAPH_CSV_TRACKER
	void PushStats(FCsvProfiler* Profiler, FTrackedData& Data)
	{
		const float Bytes = (float)((Data.BitsAccumulated+7) >> 3);
		const float KBytes = Bytes / 1000.f;

		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphKBytes), KBytes, ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphMS), static_cast<float>(Data.CPUTimeAccumulated) * 1000.f, ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphChannelsOpened), static_cast<float>(Data.ChannelsOpened), ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphNumReps), static_cast<float>(Data.NumReplications), ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphForcedUpdates), static_cast<float>(Data.ForcedUpdates), ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphCleanMS), static_cast<float>(Data.CleanCPUTimeAccumulated) * 1000.f, ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphCleanNumReps), static_cast<float>(Data.CleanNumReplications), ECsvCustomStatOp::Set);
		Profiler->RecordCustomStat(Data.StatName, CSV_CATEGORY_INDEX(ReplicationGraphRedundantMS), static_cast<float>(Data.RedundantRepCPUTimeAccumulated) * 1000.f, ECsvCustomStatOp::Set);

		Data.Reset();
	}
#endif
};

ENUM_CLASS_FLAGS(FReplicationGraphCSVTracker::EActorFlags);

// Debug Actor/Connection pair that can be set by code for further narrowing down breakpoints/logging
struct FActorConnectionPair
{
	FActorConnectionPair() { }
	FActorConnectionPair(AActor* InActor, UNetConnection* InConnection);

	TWeakObjectPtr<AActor> Actor;
	TWeakObjectPtr<UNetConnection> Connection;

	friend uint32 GetTypeHash(const FActorConnectionPair& Pair)
	{
		return HashCombine(GetTypeHash(Pair.Actor), GetTypeHash(Pair.Connection));
	}

	inline bool operator==(const FActorConnectionPair& A) const
	{
		return (A.Actor == Actor) && (A.Connection == Connection);
	}
};

// Generic/global pair that can be set by debug commands etc for extra logging/debugging functionality
extern FActorConnectionPair DebugActorConnectionPair;

/**
 * This struct will passed to all nodes in the repgraph and collect information on every ActorRepList
 */
struct FActorRepListStatCollector
{
public:
	FActorRepListStatCollector() {}

	/** Collect stats on a single FActorRepList */
	UE_API void VisitRepList(const UReplicationGraphNode* NodeToVisit, const FActorRepListRefView& RepList);

	/** Collect stats on a collection of FActorRepLists */
	UE_API void VisitStreamingLevelCollection(const UReplicationGraphNode* NodeToVisit, const struct FStreamingLevelActorListCollection& StreamingLevelList);

	/** Collect stats for FActorRepLists not held by a node */
	UE_API void VisitExplicitStreamingLevelList(FName ListOwnerName, FName StreamLevelName, const FActorRepListRefView& RepList);

	/** Prevents the node from being visited twice in case it's shared multiple times */
	UE_API void FlagNodeVisited(const UReplicationGraphNode* NodeToVisit);

	/** Print the statistics previously collected */
	UE_API void PrintCollectedData(FOutputDevice& Ar);

private:

	UE_API bool WasNodeVisited(const UReplicationGraphNode* NodeToVisit);

private:

	struct FRepListStats
	{
		uint32 NumLists = 0;
		uint32 NumActors = 0;
		uint32 MaxListSize = 0;
		uint32 NumSlack = 0;
		uint64 NumBytes = 0;
	};

	TMap<FName, FRepListStats> PerClassStats;

	TMap<FName, FRepListStats> PerStreamingLevelStats;

	/** Keeps track if a node was already collected since the same node can be shared across connections and visited multiple times */
	TMap<const UObject*, bool> VisitedNodes;
};

#undef UE_API
