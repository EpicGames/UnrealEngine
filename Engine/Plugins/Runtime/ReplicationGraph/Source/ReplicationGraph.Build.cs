// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ReplicationGraph : ModuleRules
{
	public ReplicationGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"NetCore",
				"Engine",
				"EngineSettings",
				"PerfCounters"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		SetupGameplayDebuggerSupport(Target);
	}
}
